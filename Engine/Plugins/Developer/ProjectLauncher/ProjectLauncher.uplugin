{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Project Launcher",
	"Description": "Configure custom project launch profiles.",
	"CreatedBy": "Epic Games, Inc.",
	"Category": "Platform",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"RequiresBuildPlatform": true,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"SupportedPrograms": [ "UnrealFrontend" ],
	"Modules": [
		{
			"Name": "ProjectLauncher",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			],
			"ProgramAllowList": [ "UnrealFrontend" ]
		},
		{
			"Name": "CommonLaunchExtensions",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			],
			"ProgramAllowList": [ "UnrealFrontend" ]
		}
	]
}