// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CommonLaunchExtensions : ModuleRules
{
	public CommonLaunchExtensions(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
		[
			"Core",
			"CoreUObject",
			"Json",
			"Slate",
			"SlateCore",
			"Sockets",
			"TraceLog",
			"Projects",
			"ProjectLauncher",
			"DeveloperSettings",
			"ToolWidgets",
			"Zen",
			"HTTP",
			"InputCore",
		]);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
			[
				"UnrealEd",
				"Engine"
			]);
		}

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/crashpad_handler.exe");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zen.exe");

			if (System.IO.Directory.Exists(System.IO.Path.Combine(EngineDirectory, "Binaries", "Win64", "zen.pdb")))
			{
				RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zen.pdb");
			}

			RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zenserver.exe");

			if (System.IO.Directory.Exists(System.IO.Path.Combine(EngineDirectory, "Binaries", "Win64", "zenserver.pdb")))
			{
				RuntimeDependencies.Add("$(EngineDir)/Binaries/Win64/zenserver.pdb");
			}

			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/win-x64/OidcToken.exe");
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Linux/zen");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Linux/zenserver");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/linux-x64/OidcToken");
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/crashpad_handler");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/zen");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/Mac/zenserver");
			RuntimeDependencies.Add("$(EngineDir)/Binaries/DotNET/OidcToken/osx-x64/OidcToken");
		}
	}
}
