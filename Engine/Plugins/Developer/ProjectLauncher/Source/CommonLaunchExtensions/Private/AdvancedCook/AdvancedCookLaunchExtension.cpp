// Copyright Epic Games, Inc. All Rights Reserved.

#include "AdvancedCook/AdvancedCookLaunchExtension.h"
#include "Widgets/Shared/SCustomLaunchMapListView.h"
#include "Widgets/Shared/SCustomLaunchCultureListView.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "SResizeBox.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "FAdvancedCookLaunchExtensionInstance"



FAdvancedCookLaunchExtensionInstance::FAdvancedCookLaunchExtensionInstance( FArgs& InArgs ) 
	: ProjectLauncher::FBuildCookRunCommandExtensionInstance(InArgs)
	, Owner(StaticCastSharedRef<FAdvancedCookLaunchExtension>(InArgs.Extension))
{
}

TSharedRef<ProjectLauncher::FBuildCookRunExtension> FAdvancedCookLaunchExtensionInstance::CreateBuildCookRunExtension( const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs )
{
	return MakeShared<FBuildCookRunInstance>(InArgs);
}


bool FAdvancedCookLaunchExtensionInstance::IsBuildCookRunExtensionEnabledByDefault( const ILauncherProfileBuildCookRunRef& InBuildCookRun ) const
{
	return Owner->HasAdvancedCookOptions(InBuildCookRun);
}

bool FAdvancedCookLaunchExtensionInstance::CanToggleBuildCookRunExtension( const ILauncherProfileBuildCookRunRef& InBuildCookRun, bool bWantToEnable ) const
{
	if (!bWantToEnable && Owner->HasAdvancedCookOptions(InBuildCookRun))
	{
		return false;
	}

	return true;
}




FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::FBuildCookRunInstance( const FArgs& InArgs )
	: ProjectLauncher::FBuildCookRunExtension( InArgs )
{
	MapOption = GetBuildCookRun()->GetCookedMaps().Num() > 0 ? EMapOption::Selected : EMapOption::Startup;

	CachedCulturesToCook = GetBuildCookRun()->GetCookedCultures();
	CultureOption = (CachedCulturesToCook.Num() == 1 && CachedCulturesToCook.Contains(TEXT("en"))) ? ECultureOption::Default : ECultureOption::Custom;

	LoadMapFilterState();
}


void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::CustomizeTree( ProjectLauncher::FLaunchProfileTreeNode& ProfileTreeNode )
{
	auto IsVisible = [this]()
	{
		return (GetBuildCookRun()->GetCookMode() == ELauncherProfileCookModes::ByTheBook);
	};

	AddDefaultHeading(ProfileTreeNode)
		.AddBoolean( LOCTEXT("UnversionedCookLabel", "Unversioned Cook"),
			{
				.GetValue = [this]()			{ return GetBuildCookRun()->IsCookingUnversioned(); },
				.SetValue = [this](bool bValue)	{ GetBuildCookRun()->SetUnversionedCooking(bValue); },
				.GetDefaultValue = [this]()		{ return GetDefaultBuildCookRun()->IsCookingUnversioned(); },
				.IsVisible = IsVisible,
				.Validation = ProjectLauncher::FValidation({ELauncherProfileValidationErrors::UnversionedAndIncremental}),
			}
		)
		.AddWidget( LOCTEXT("MapsToCookLabel", "Maps To Cook"), 
			{
				.IsVisible = IsVisible,
			},
			CreateMapListWidget()
		)
		.AddWidget( LOCTEXT("CulturesToCookLabel", "Cultures To Cook"), 
			{
				.IsVisible = IsVisible,
			},
			CreateCultureListWidget()
		)
		.AddString( LOCTEXT("AdditionalCookerOptionsLabel", "Additional Cooker Options"), 
			{
				.GetValue = [this]()				{ return GetBuildCookRun()->GetCookOptions(); },
				.SetValue = [this](FString Value)	{ GetBuildCookRun()->SetCookOptions( Value ); },
				.GetDefaultValue = []()				{ return FString(); },
				.IsVisible = IsVisible,
			}
		)
	;
}




BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::CreateMapListWidget()
{
	MapListView = 
		SNew(SCustomLaunchMapListView, GetModel())
		.OnSelectionChanged(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetMapsToCook)
		.SelectedMaps(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetMapsToCook)
		.ProjectPath_Lambda( [this]() { return GetModel()->GetProjectPath(GetProfile()); })
		.OnMapFilterChanged(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::OnMapFilterChanged)
		.MapSourceFlags(MapSourceFlags)
	;

	return SNew(SVerticalBox)

		// map option controls
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)

			// map option
			+SHorizontalBox::Slot()
			.Padding(0, 0)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SSegmentedControl<EMapOption>)
				.Value( this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetMapOption)
				.OnValueChanged(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetMapOption)

				+SSegmentedControl<EMapOption>::Slot(EMapOption::Startup)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("StartupMapsLabel", "Startup Maps"))
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				]

				+SSegmentedControl<EMapOption>::Slot(EMapOption::Selected)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SelectedMapsLabel", "Selected Maps"))
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				]
			]

			// map selector controls (search etc)
			+SHorizontalBox::Slot()
			.Padding(8, 0)
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Visibility_Lambda([this](){ return GetMapOption() == EMapOption::Selected  ? EVisibility::Visible : EVisibility::Collapsed; } )
				[
					MapListView->MakeControlsWidget()
				]
			]
		]

		// map list
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SVerticalResizeBox)
			.Visibility_Lambda([this](){ return (GetMapOption() == EMapOption::Selected) ? EVisibility::Visible : EVisibility::Collapsed; } )
			.HandleHeight(4)
			.ContentHeight(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetMapListHeight)
			.ContentHeightChanged(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetMapListHeight)
			.HandleColor( FAppStyle::Get().GetSlateColor("Colors.Secondary").GetSpecifiedColor() )
			[
				MapListView.ToSharedRef()
			]
		]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::CreateCultureListWidget()
{
	CultureListView = 
		SNew(SCustomLaunchCultureListView, GetModel())
		.OnSelectionChanged(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetCulturesToCook)
		.SelectedCultures(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetCulturesToCook)
		.ProjectPath_Lambda( [this]() { return GetModel()->GetProjectPath(GetProfile()); })
	;

	return SNew(SVerticalBox)

		// culture option controls
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,2)
		[
			SNew(SHorizontalBox)

			// culture option
			+SHorizontalBox::Slot()
			.Padding(0, 0)
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SSegmentedControl<ECultureOption>)
				.Value( this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetCultureOption)
				.OnValueChanged(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetCultureOption)

				+SSegmentedControl<ECultureOption>::Slot(ECultureOption::Default)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DefaultCultureLabel", "Default"))
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				]

				+SSegmentedControl<ECultureOption>::Slot(ECultureOption::Custom)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("CustomCultureLabel", "Custom"))
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				]
			]

			// culture selector controls (search etc)
			+SHorizontalBox::Slot()
			.Padding(8, 0)
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.Visibility_Lambda([this](){ return GetCultureOption() == ECultureOption::Custom ? EVisibility::Visible : EVisibility::Collapsed; } )
				[
					CultureListView->MakeControlsWidget()
				]
			]
		]

		// culture list
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SVerticalResizeBox)
			.Visibility_Lambda([this](){ return GetCultureOption() == ECultureOption::Custom ? EVisibility::Visible : EVisibility::Collapsed; } )
			.HandleHeight(4)
			.ContentHeight(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetCultureListHeight)
			.ContentHeightChanged(this, &FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetCultureListHeight)
			.HandleColor( FAppStyle::Get().GetSlateColor("Colors.Secondary").GetSpecifiedColor() )
			[
				CultureListView.ToSharedRef()
			]
		]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetMapsToCook(TArray<FString> MapsToCook)
{
	GetBuildCookRun()->ClearCookedMaps(); 
	for (const FString& Map : MapsToCook)
	{
		GetBuildCookRun()->AddCookedMap(Map);
	}

	GetExtensionInstance()->BroadcastPropertyChanged();
}

TArray<FString> FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetMapsToCook() const
{
	return GetBuildCookRun()->GetCookedMaps();
}

float FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetMapListHeight() const
{
	return MapListHeight;
}

void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetMapListHeight( float NewHeight )
{
	static const float MinMapListHeight = 100.0f;

	MapListHeight = FMath::Max(NewHeight, MinMapListHeight);

	GetExtensionInstance()->RequestTreeRefresh();
}

FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::EMapOption FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetMapOption() const
{
	return MapOption;
}

void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetMapOption( EMapOption NewMapOption )
{
	bool bShow = (NewMapOption == EMapOption::Selected);

	MapOption = NewMapOption;

	if (bShow)
	{
		// restore the cooked maps again, if any
		if (CachedMapsToCook.Num() > 0 && GetBuildCookRun()->GetCookedMaps().Num() == 0)
		{
			SetMapsToCook(CachedMapsToCook);
			CachedMapsToCook.Reset();
		}
	}
	else
	{
		// to set the 'cook startup maps only', its necessary to remove all the cooked maps - take a copy of the values to allow it to be restored
		CachedMapsToCook = GetBuildCookRun()->GetCookedMaps();
		SetMapsToCook(TArray<FString>());
	}

	GetExtensionInstance()->BroadcastPropertyChanged();

	MapListView->RefreshMapList();
}




void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetCulturesToCook(TArray<FString> CulturesToCook)
{
	GetBuildCookRun()->ClearCookedCultures(); 
	for (const FString& Culture : CulturesToCook)
	{
		GetBuildCookRun()->AddCookedCulture(Culture);
	}

	GetExtensionInstance()->BroadcastPropertyChanged();
}

TArray<FString> FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetCulturesToCook() const
{
	return GetBuildCookRun()->GetCookedCultures();
}

float FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetCultureListHeight() const
{
	return CultureListHeight;
}

void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetCultureListHeight( float NewHeight )
{
	static const float MinCultureListHeight = 100.0f;

	CultureListHeight = FMath::Max(NewHeight, MinCultureListHeight);

	GetExtensionInstance()->RequestTreeRefresh();
}

FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::ECultureOption FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::GetCultureOption() const
{
	return CultureOption;
}

void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SetCultureOption( ECultureOption NewCultureOption )
{
	bool bShow = (NewCultureOption == ECultureOption::Custom);

	CultureOption = NewCultureOption;

	if (bShow)
	{
		// restore the cooked cultures again
		SetCulturesToCook(CachedCulturesToCook);
		CachedCulturesToCook.Reset();
	}
	else
	{
		// to set the 'english only', its necessary to remove all the currently selected cultures - take a copy of the values to allow it to be restored
		CachedCulturesToCook = GetBuildCookRun()->GetCookedCultures();
		SetCulturesToCook(TArray<FString>{TEXT("en")});
	}

	GetExtensionInstance()->BroadcastPropertyChanged();

	CultureListView->RefreshCultureList();
}


void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::OnProjectChanged()
{
	if (MapListView.IsValid())
	{
		MapListView->RefreshMapList();
	}
}

struct FMapFilterEntry 
{
	EMapSourceFlags Flag = EMapSourceFlags::None;
	const TCHAR* Key = nullptr;
};
static const FMapFilterEntry MapFilterEntries[] =
{
	{ EMapSourceFlags::MyDeveloperContent,    TEXT("MapFilter_MyDeveloperContent") },
	{ EMapSourceFlags::OtherDeveloperContent, TEXT("MapFilter_OtherDeveloperContent") },
	{ EMapSourceFlags::ProjectPlugins,        TEXT("MapFilter_ProjectPluginMaps") },
	{ EMapSourceFlags::EngineContent,         TEXT("MapFilter_EngineContent") },
	{ EMapSourceFlags::EnginePlugins,         TEXT("MapFilter_EnginePluginMaps") },
};

void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::OnMapFilterChanged(EMapSourceFlags NewFlags)
{
	MapSourceFlags = NewFlags;
	SaveMapFilterState();
	GetExtensionInstance()->BroadcastPropertyChanged();
}

void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::LoadMapFilterState()
{
	const TMap<FString, bool>& CustomBools = GetProfile()->GetCustomBoolProperties();
	for (const FMapFilterEntry& Entry : MapFilterEntries)
	{
		if (const bool* ValuePtr = CustomBools.Find(Entry.Key))
		{
			if (*ValuePtr)
			{
				MapSourceFlags |= Entry.Flag;
			}
			else
			{
				MapSourceFlags &= ~Entry.Flag;
			}
		}
	}
}

void FAdvancedCookLaunchExtensionInstance::FBuildCookRunInstance::SaveMapFilterState()
{
	TMap<FString, bool>& CustomBools = GetProfile()->GetCustomBoolProperties();
	for (const FMapFilterEntry& Entry : MapFilterEntries)
	{
		CustomBools.Add(Entry.Key, EnumHasAnyFlags(MapSourceFlags, Entry.Flag));
	}
}

TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FAdvancedCookLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
{
	return MakeShared<FAdvancedCookLaunchExtensionInstance>(InArgs);
}

const TCHAR* FAdvancedCookLaunchExtension::GetInternalName() const
{
	return TEXT("AdvancedCook");
}

FText FAdvancedCookLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Advanced Cook Options");
}

bool FAdvancedCookLaunchExtension::IsCreatedByDefault( ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel ) const
{
	return HasAdvancedCookOptions(InProfile);
}

bool FAdvancedCookLaunchExtension::HasAdvancedCookOptions(const ILauncherProfileBuildCookRunRef& InBuildCookRun) const
{
	if (!InBuildCookRun->IsCookingUnversioned() || !InBuildCookRun->GetCookOptions().IsEmpty() || InBuildCookRun->GetCookedMaps().Num() > 0)
	{
		return true;
	}

	bool bHasDefaultCultures = (InBuildCookRun->GetCookedCultures().Num() == 1 && InBuildCookRun->GetCookedCultures()[0] == TEXT("en"));
	if (!bHasDefaultCultures)
	{
		return true;
	}

	// we're not using any advanced cook options
	return false;
}

bool FAdvancedCookLaunchExtension::HasAdvancedCookOptions(ILauncherProfileRef InProfile) const
{
	for (const ILauncherProfileBuildCookRunRef& BuildCookRun : InProfile->GetBuildCookRunCommands())
	{
		if (HasAdvancedCookOptions(BuildCookRun))
		{
			return true;
		}
	}

	return false;
}




#undef LOCTEXT_NAMESPACE
