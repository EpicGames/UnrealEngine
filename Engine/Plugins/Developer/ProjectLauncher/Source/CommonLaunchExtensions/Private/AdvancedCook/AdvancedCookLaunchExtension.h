// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/BuildCookRunCommandExtension.h"
#include "Widgets/Shared/SCustomLaunchMapListView.h"


class SCustomLaunchMapListView;
class SCustomLaunchCultureListView;


class FAdvancedCookLaunchExtensionInstance : public ProjectLauncher::FBuildCookRunCommandExtensionInstance
{
public:
	FAdvancedCookLaunchExtensionInstance( FArgs& InArgs );
	virtual ~FAdvancedCookLaunchExtensionInstance() = default;

	virtual TSharedRef<ProjectLauncher::FBuildCookRunExtension> CreateBuildCookRunExtension( const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs ) override;

	virtual bool IsBuildCookRunExtensionEnabledByDefault( const ILauncherProfileBuildCookRunRef& InBuildCookRun ) const override;
	virtual bool CanToggleBuildCookRunExtension( const ILauncherProfileBuildCookRunRef& InBuildCookRun, bool bWantToEnable ) const override;


protected:
	TSharedRef<class FAdvancedCookLaunchExtension> Owner;

	class FBuildCookRunInstance : public ProjectLauncher::FBuildCookRunExtension
	{
	public:
		FBuildCookRunInstance(const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs );
		virtual void CustomizeTree( ProjectLauncher::FLaunchProfileTreeNode& ProfileTreeNode ) override;


		TArray<FString> CachedMapsToCook;
		enum class EMapOption : uint8
		{
			Startup,
			Selected,
		};
		EMapOption MapOption = EMapOption::Startup;

		void SetMapsToCook(TArray<FString> MapsToCook);
		TArray<FString> GetMapsToCook() const;
		EMapOption GetMapOption() const;
		void SetMapOption( EMapOption MapOption );
		float GetMapListHeight() const;
		void SetMapListHeight( float NewHeight );
		TSharedRef<SWidget> CreateMapListWidget();

		float MapListHeight = 300.0f;
		TSharedPtr<SCustomLaunchMapListView> MapListView;

		// Map filter state
		EMapSourceFlags MapSourceFlags = EMapSourceFlags::None;
		void OnMapFilterChanged(EMapSourceFlags NewFlags);
		void LoadMapFilterState();
		void SaveMapFilterState();


		TArray<FString> CachedCulturesToCook;
		enum class ECultureOption : uint8
		{
			Default,
			Custom,
		};
		ECultureOption CultureOption = ECultureOption::Default;

		void SetCulturesToCook(TArray<FString> CulturesToCook);
		TArray<FString> GetCulturesToCook() const;
		ECultureOption GetCultureOption() const;
		void SetCultureOption( ECultureOption CultureOption );
		float GetCultureListHeight() const;
		void SetCultureListHeight( float NewHeight );
		TSharedRef<SWidget> CreateCultureListWidget();

		float CultureListHeight = 150.0f;
		TSharedPtr<SCustomLaunchCultureListView> CultureListView;


		virtual void OnProjectChanged() override;
	};
};


class FAdvancedCookLaunchExtension : public ProjectLauncher::FBuildCookRunCommandExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
	virtual bool IsCreatedByDefault( ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel ) const override;

	bool HasAdvancedCookOptions(const ILauncherProfileBuildCookRunRef& InBuildCookRun) const;
	bool HasAdvancedCookOptions(ILauncherProfileRef InProfile) const;
};