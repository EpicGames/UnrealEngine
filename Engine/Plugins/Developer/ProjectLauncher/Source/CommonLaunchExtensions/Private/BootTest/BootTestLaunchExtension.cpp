// Copyright Epic Games, Inc. All Rights Reserved.

#include "BootTest/BootTestLaunchExtension.h"


#define LOCTEXT_NAMESPACE "FBootTestLaunchExtensionInstance"

void FBootTestLaunchExtensionInstance::CustomizeTree( ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData )
{
	// note: Windowed option is here mostly as an example of adding UI items for a particular profile - your tests may not need it

	AddDefaultHeading(ProfileTreeData)
		.AddBoolean(LOCTEXT("WindowedLabel","Windowed"),
		{
			.GetValue = [this]()          { return GetConfigBool(EConfig::PerProfile, TEXT("Windowed")); },
			.SetValue = [this](bool bVal) { SetConfigBool(EConfig::PerProfile, TEXT("Windowed"), bVal ); },
		}
	);
}

void FBootTestLaunchExtensionInstance::CustomizeUATCommandLine( FString& InOutCommandLine )
{
	bool bWindowed = GetConfigBool(EConfig::PerProfile, TEXT("Windowed"));
	if (!bWindowed)
	{
		InOutCommandLine += TEXT(" -windowmode=Fullscreen");
	}
}

void FBootTestLaunchExtensionInstance::OnTestAdded( ILauncherProfileAutomatedTestRef AutomatedTest )
{
	AutomatedTest->SetTests(TEXT("UE.BootTest"));
}

TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FBootTestLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
{
	return MakeShared<FBootTestLaunchExtensionInstance>(InArgs);
}

const TCHAR* FBootTestLaunchExtension::GetInternalName() const
{
	return TEXT("BootTest");
}

FText FBootTestLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Automated Boot Test");
}


#undef LOCTEXT_NAMESPACE
