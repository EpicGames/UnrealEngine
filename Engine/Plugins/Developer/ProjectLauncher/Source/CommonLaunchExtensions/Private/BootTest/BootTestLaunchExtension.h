// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/AutomatedTestLaunchExtension.h"



class FBootTestLaunchExtensionInstance : public ProjectLauncher::FAutomatedTestLaunchExtensionInstance
{
public:
	FBootTestLaunchExtensionInstance( FArgs& InArgs ) : ProjectLauncher::FAutomatedTestLaunchExtensionInstance(InArgs) {};
	virtual ~FBootTestLaunchExtensionInstance() = default;

	virtual void OnTestAdded( ILauncherProfileAutomatedTestRef AutomatedTest ) override;

	virtual void CustomizeTree( ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData ) override;
	virtual void CustomizeUATCommandLine( FString& InOutCommandLine ) override;

private:
	static const TCHAR* BootTestInternalName;

};


class FBootTestLaunchExtension : public ProjectLauncher::FAutomatedTestLaunchExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
};