// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildSync/BuildInfoHelper.h"
#include "Misc/EngineVersion.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "FBuildSyncLaunchExtensionInstance"



const TCHAR* FBuildInfoHelper::DefaultBuildType = TEXT("staged-build");


FBuildInfoHelper::FBuildInfoHelper()
	: bBuildListRetrieverInitComplete(false)
{
	BuildListRetriever = MakeShared<UE::Zen::Build::FBuildListRetriever>();
	BackendRetriever = MakeShared<FBackendRetriever>();
	UGSBuildInfoRetriever = MakeShared<FUGSBuildInfoRetriever>();

	BuildType = DefaultBuildType;


	for ( const auto& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
	{
		const FDataDrivenPlatformInfo& DDPI = Pair.Value;
		if (!DDPI.bIsFakePlatform)
		{
			KnownPlatformNames.Add(DDPI.IniPlatformName.ToString().ToLower());
			KnownPlatformNames.Add(DDPI.UBTPlatformName.ToString().ToLower());
		}
	}
}

FBuildInfoHelper::~FBuildInfoHelper()
{
	BuildListRetriever->SetOnReadyForQuery(nullptr);
	BuildListRetriever.Reset();
}

void FBuildInfoHelper::Connect()
{
	if (!bBuildListRetrieverInitComplete)
	{
		auto OnReadyForQuery = [ThisRef = AsShared()]()
		{
			ThisRef->Refresh(false);
		};
		BuildListRetriever->SetOnReadyForQuery(OnReadyForQuery);
		bBuildListRetrieverInitComplete = true;
	}

	BuildListRetriever->ConnectToBuildService();
}



bool FBuildInfoHelper::IsConnected() const
{
	return BuildListRetriever->IsConnected();
}



bool FBuildInfoHelper::IsConfigured() const
{
	return BackendRetriever->IsConfigured() && UGSBuildInfoRetriever->IsConfigured();
}

FText FBuildInfoHelper::GetErrorText() const
{
	if (!BackendRetriever->IsConfigured() && !UGSBuildInfoRetriever->IsConfigured())
	{
		return LOCTEXT("RetrieversUnconfigured", "Horde and Backend URLs are not set in the Engine.ini");
	}
	else if (!BackendRetriever->IsConfigured())
	{
		return LOCTEXT("BackendUnconfigured", "Backend URL is not configured in the Engine.ini");
	}
	else if (!UGSBuildInfoRetriever->IsConfigured())
	{
		return LOCTEXT("UGSUnconfigured", "UGS URL is not configured in the Engine.ini");
	}
	else
	{
		return LOCTEXT("NoError", "Configured");
	}
}

bool FBuildInfoHelper::IsRefreshing() const
{
	return bIsRefreshing || !BuildListRetriever->IsReadyForQuery();
}



void FBuildInfoHelper::Refresh( bool bFullRefresh )
{
	if (bFullRefresh)
	{
		BuildInfos.Reset();
		BuildListRetriever->Refresh();
	}
	else if (!bIsRefreshing)
	{
		bIsRefreshing = true;

		RefreshBackend();
		RefreshBuildGroups();
	}
}



void FBuildInfoHelper::CheckRefreshComplete()
{
	if ( !(bBuildGroupsRequestComplete && !BackendRetriever->IsGettingBackends() && !UGSBuildInfoRetriever->IsGettingUGSBuildInfo()))
	{
		return;
	}

	RebuildBuildInfos();

	ExecuteOnGameThread(UE_SOURCE_LOCATION, [ThisRef = AsShared()]
	{
		if (ThisRef->OnBuildsRefreshed.IsSet())
		{
			ThisRef->OnBuildsRefreshed();
		}
		ThisRef->bIsRefreshing = false;
	});
}



void FBuildInfoHelper::SetFilter( const FFilter& InFilter )
{
	Filter = InFilter;
	Refresh(false);
}



void FBuildInfoHelper::SetProjectName( const FString& InProjectName )
{
	if (ProjectName != InProjectName)
	{
		ProjectName = InProjectName;

		if (IsConnected())
		{
			Refresh(false);
		}
	}
}

void FBuildInfoHelper::SetBuildType( const FString& InBuildType )
{
	if (BuildType != InBuildType)
	{
		BuildType = InBuildType;

		if (IsConnected())
		{
			Refresh(false);
		}
	}
}

FString FBuildInfoHelper::GetBuildType() const
{
	return BuildType;
}

void FBuildInfoHelper::SetBuildsRefreshedHandler(TFunction<void()> InOnBuildsRefreshed)
{
	OnBuildsRefreshed = InOnBuildsRefreshed;
}


void FBuildInfoHelper::RefreshBuildGroups()
{
	FilteredBuildGroups.Reset();
	bBuildGroupsRequestComplete = false;

	auto OnComplete = [ThisRef = AsShared()]()
	{
		ThisRef->bBuildGroupsRequestComplete = true;
		
		ThisRef->RefreshUGSBuildInfo(); // UGS build info requires the list of CLs from the build detail
		ThisRef->CheckRefreshComplete();
	};

	// prepare the build detail query
	FString Branch = FEngineVersion::Current().GetBranch();
	Branch = Branch.Replace(TEXT("//"), TEXT("")).Replace(TEXT("/"), TEXT("-")).ToLower();

	// launch the build query
	bool bQuerying = BuildListRetriever->QueryBuildGroups(ProjectName, BuildType, Branch, 
		[ThisRef = AsShared(), OnComplete](const TArrayView<TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildGroup>>& InBuildGroups) mutable
		{
			ThisRef->CreateFilteredBuildGroups(InBuildGroups);
			OnComplete();
		}
	);
	if (!bQuerying)
	{
		OnComplete();
	}
}



void FBuildInfoHelper::CreateFilteredBuildGroups( const TArrayView<TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildGroup>>& BuildGroups )
{
	FilteredBuildGroups.Reset();

	for (TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildGroup> BuildGroup : BuildGroups)
	{
		if (FilteredBuildGroups.Num() >= Filter.MaxItems)
		{
			break;
		}

		if (!Filter.MaxAge.IsZero() && (FDateTime::Now() - BuildGroup->CreatedAt) > Filter.MaxAge)
		{
			continue;
		}

		FilteredBuildGroups.Add(BuildGroup);
	}
}



void FBuildInfoHelper::RebuildBuildInfos()
{
	BuildInfos.Reset(FilteredBuildGroups.Num());
	AllKnownNamedArtifacts.Reset();

	for ( TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildGroup> BuildGroup : FilteredBuildGroups)
	{
		TSharedPtr<FBuildInfo> BuildInfo = MakeShared<FBuildInfo>();
		BuildInfo->Group = BuildGroup;

		const TArray<FString>* BackendsPtr = BackendRetriever->GetBuildToBackendsMap().Find(BuildGroup->Changelist);
		if (BackendsPtr != nullptr)
		{
			BuildInfo->Backends = (*BackendsPtr);
		}

		const TSharedRef<FUGSBuildInfoRetriever::FUGSBuildInfo>* UGSBuildInfoPtr = UGSBuildInfoRetriever->GetBuildToUGSBuildInfoMap().Find(BuildGroup->Changelist);
		if (UGSBuildInfoPtr != nullptr)
		{
			BuildInfo->UGSBuildInfo = (*UGSBuildInfoPtr);
		}

		for ( const auto& Pair : BuildGroup->NamedArtifacts)
		{
			const FString& ArtifactName = Pair.Key; // @fixme: these are only 'platform' strings by convention unfortunately
			const UE::Zen::Build::FBuildServiceInstance::FBuildRecord& BuildRecord = Pair.Value;
			
			FString Platform = GetPlatformNameFromNamedArtifact(ArtifactName, &BuildRecord);

			FName PlatformName(Platform);
			BuildInfo->Platforms.Add(PlatformName);

			if (!BuildInfo->PlatformToArtifacts.Contains(PlatformName))
			{
				BuildInfo->PlatformToArtifacts.Add(PlatformName);
			}
			BuildInfo->PlatformToArtifacts[PlatformName].Add(ArtifactName);

			AllKnownNamedArtifacts.Add(ArtifactName);
		}
		BuildInfo->Platforms.Sort( []( const FName& A, const FName& B )
		{
			return A.ToString() < B.ToString();
		});

		BuildInfos.Add(BuildInfo);
	}

	BuildTypesMap = BuildListRetriever->GenerateBuildTypesMap();
}



void FBuildInfoHelper::RefreshUGSBuildInfo()
{
	TArray<int32> Changelists;
	for (TSharedPtr<UE::Zen::Build::FBuildListRetriever::FBuildGroup> BuildGroup : FilteredBuildGroups)
	{
		Changelists.AddUnique(BuildGroup->Changelist);
	}

	auto OnComplete = [ThisRef = AsShared()]()
	{
		ThisRef->CheckRefreshComplete();
	};
	UGSBuildInfoRetriever->GetUGSBuildInfoAsync(ProjectName, Changelists, OnComplete);
}



void FBuildInfoHelper::RefreshBackend()
{
	auto OnComplete = [ThisRef = AsShared()]()
	{
		ThisRef->CheckRefreshComplete();
	};

	BackendRetriever->GetBackendsAsync(OnComplete);
}



TSharedRef<UE::Zen::Build::FBuildServiceInstance> FBuildInfoHelper::GetServiceInstance() const
{
	return BuildListRetriever->GetServiceInstance();
}



const TArray<TSharedPtr<FBuildInfoHelper::FBuildInfo, ESPMode::ThreadSafe>>& FBuildInfoHelper::GetBuildInfos() const
{
	return BuildInfos;
}

const TMap<FString,TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildType, ESPMode::ThreadSafe>>& FBuildInfoHelper::GetBuildTypesMap() const
{
	return BuildTypesMap;
}

const TSet<FString>& FBuildInfoHelper::GetAllKnownNamedArtifacts() const
{
	return AllKnownNamedArtifacts;
}


FString FBuildInfoHelper::GetPlatformNameFromNamedArtifact( const FString& ArtifactName, const UE::Zen::Build::FBuildServiceInstance::FBuildRecord* BuildRecord ) const
{
	// build record's platform and the artifact name is only related to the UnrealTargetPlatform by convention :-(

	// use the metadata platorm name if we have it, otherwise the artifact name will have to do
	FString PlatformSearchString = ArtifactName;
	if (BuildRecord != nullptr)
	{
		if (FCbFieldView CookPlatformField = BuildRecord->Metadata.FindIgnoreCase("platform"); CookPlatformField.HasValue() && !CookPlatformField.HasError())
		{
			FString CookPlatform(CookPlatformField.AsString());
			if (!CookPlatform.IsEmpty())
			{
				PlatformSearchString = CookPlatform;
			}
		}
	}


	auto IsPlatform = [this](const FString& Value)
	{
		return KnownPlatformNames.Contains(Value.ToLower());
	};

	auto TrySetPlatform = [IsPlatform](FString Value)
	{
		Value.RemoveFromEnd(TEXT("Client"));
		Value.RemoveFromEnd(TEXT("Game"));
		Value.RemoveFromEnd(TEXT("Server"));
		Value.RemoveFromEnd(TEXT("Editor"));

		if (IsPlatform(Value))
		{
			Value.ReplaceInline(TEXT("Win64"), TEXT("Windows"));
			return Value;
		}

		return FString();
	};


	auto TrySetPlatformFromItems = [TrySetPlatform](const TArray<FString>& Items)
	{
		for (const FString& Item : Items)
		{
			FString Platform = TrySetPlatform(Item);
			if (!Platform.IsEmpty())
			{
				return Platform;
			}
		}

		return FString();
	};



	FString Platform;

	Platform = TrySetPlatform(PlatformSearchString); // raw Platform
	if (!Platform.IsEmpty())
	{
		return Platform;
	}

	TArray<FString> DashedItems;
	PlatformSearchString.ParseIntoArray(DashedItems, TEXT("-"));
	Platform = TrySetPlatformFromItems(DashedItems); // Platform-BuildType-TargetType etc
	if (!Platform.IsEmpty())
	{
		return Platform;
	}

	TArray<FString> UnderscoredItems;
	PlatformSearchString.ParseIntoArray(UnderscoredItems, TEXT("_"));
	Platform = TrySetPlatformFromItems(UnderscoredItems); // Platform_Flavor etc
	if (!Platform.IsEmpty())
	{
		return Platform;
	}


	// cannot infer platform - try some fallbacks

	if (DashedItems.Num() > 1)
	{
		return DashedItems[0];
	}

	if (UnderscoredItems.Num() > 1)
	{
		return UnderscoredItems[0];
	}

	return PlatformSearchString;
}



#undef LOCTEXT_NAMESPACE

