// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildSync/BuildSyncLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"
#include "Misc/EngineVersion.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "PlatformInfo.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "DesktopPlatformModule.h"
#include "Modules/BuildVersion.h"
#include "Interfaces/IProjectManager.h"
#include "Widgets/Shared/SCustomLaunchCombo.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Internationalization/Regex.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "BuildSync/SBuildSyncBuildCombo.h"
#include "Extension/BuildCookRunLaunchExtension.h"

#define LOCTEXT_NAMESPACE "FBuildSyncLaunchExtensionInstance"


FBuildSyncLaunchExtensionInstance::FBuildSyncLaunchExtensionInstance( FArgs& InArgs ) 
	: ProjectLauncher::FCustomUATCommandLaunchExtensionInstance(InArgs) 
	, Owner(StaticCastSharedRef<FBuildSyncLaunchExtension>(InArgs.Extension))
{
	// cache platform information
	GetConfigString(EConfig::PerProfile, TEXT("NamedArtifacts")).ParseIntoArray(CachedNamedArtifacts, TEXT(";"));
	CachedCookPlatforms = GetCookPlatforms();

	DestinationDirectory = GetConfigString(EConfig::PerProfile, TEXT("DestinationDirectory"));
	if (DestinationDirectory.IsEmpty())
	{
		FString CookedDir = FPaths::Combine(*FPaths::RootDir(), *InArgs.Profile->GetProjectBasePath(), TEXT("Saved"), TEXT("StagedBuilds"));
		SetDestinationDirectory(CookedDir);
	}

	// create build info helper
	BuildInfo = MakeShared<FBuildInfoHelper>();
	BuildInfo->SetProjectName(GetProfile()->GetProjectName());
	BuildInfo->SetBuildType(GetConfigString(EConfig::PerProfile, TEXT("BuildType"), FBuildInfoHelper::DefaultBuildType)); // do we want to set the default build type based on the content scheme?

	auto OnRefreshed = [this]()
	{
		OnBuildsRefreshed();
	};
	BuildInfo->SetBuildsRefreshedHandler(OnRefreshed);

	GetProfile()->OnUATCommandAdded().AddRaw(this, &FBuildSyncLaunchExtensionInstance::OnOtherUATCommandAdded);
}

FBuildSyncLaunchExtensionInstance::~FBuildSyncLaunchExtensionInstance()
{
	BuildInfo->SetBuildsRefreshedHandler(nullptr);
	BuildInfo.Reset();
	BuildCombo.Reset();

	GetProfile()->OnUATCommandAdded().RemoveAll(this);
}


void FBuildSyncLaunchExtensionInstance::CustomizeTree( ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData )
{
	FLaunchExtensionInstance::CustomizeTree(ProfileTreeData);

	using namespace UE::Zen::Build;

	auto IsEnabled = []()
	{
		return !GIsEditor;
	};

	auto IsEnabledAndReady = [this]()
	{
		return BuildInfo->IsConnected() && !BuildInfo->IsRefreshing() && !GIsEditor;
	};

	auto IsSkipContentInStageEnabled = [this]()
	{
		return GetBuildType() == TEXT("staged-build");
	};

	AddDefaultHeading(ProfileTreeData)
		.AddWidget( LOCTEXT("ConnectionLabel", "Connection"),
			{
				.IsVisible = [this]() { return GetBuildServiceConnectionState() != FBuildServiceInstance::EConnectionState::ConnectionSucceeded; },
				.IsEnabled = IsEnabled,
			},
			CreateBuildStorageConnectionWidget()
		)
		.AddWidget( LOCTEXT("BuildTypeLabel", "Build Type"),
			{
				.IsEnabled = IsEnabledAndReady,
			},
			CreateBuildTypeSelectionWidget()
		)
		.AddBoolean(LOCTEXT("SkipContentLabel", "Skip Content"), 
			{
				.GetValue = [this]() { return GetConfigBool(EConfig::PerProfile, TEXT("SkipContentInStaged")); },
				.SetValue = [this](bool bEnable){ SetConfigBool(EConfig::PerProfile, TEXT("SkipContentInStaged"), bEnable); },
				.IsVisible = IsSkipContentInStageEnabled,
				.IsEnabled = IsEnabledAndReady,
			},
			LOCTEXT("SkipContentDescription", "Download staged build without cooked content")
		)
		.AddWidget( LOCTEXT("BuildLabel", "Build"),
			{
				.IsEnabled = IsEnabledAndReady,
				.Validation = ProjectLauncher::FValidation({TEXT("Validation_BuildSync_NoBuild")}),
			},
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(1,1)
			.VAlign(VAlign_Center)
			[			
				SAssignNew(BuildCombo, SBuildSyncBuildCombo)
				.OptionsSource(&FilteredBuildInfos)
				.OnSelectionChanged(this, &FBuildSyncLaunchExtensionInstance::SetSelectedBuild)
				.SelectedItem(this, &FBuildSyncLaunchExtensionInstance::GetSelectedBuild)
				.GetItemSuitability(this, &FBuildSyncLaunchExtensionInstance::GetBuildSuitability)
				.FilterWidget(CreateBuildFilterWidget())
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4,1)
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ToolTipText(LOCTEXT("RefreshTip", "Refresh the list of available builds"))
				.OnClicked_Lambda([this]()
				{ 
					SelectedBuild.Reset(); // clear the build before doing a deep refresh - this means we auto-select the most suitable build
					BuildInfo->Refresh(true); 
					return FReply::Handled(); 
				})
				.Content()
				[
					SNew(SBox)
					.WidthOverride(12)
					.HeightOverride(12)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush("Icons.Refresh"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				]
			]
		)
		.AddWidget(LOCTEXT("NamedArtifactLabel", "Build Platform"),
			{
				.IsEnabled = IsEnabledAndReady,
				.Validation = ProjectLauncher::FValidation({TEXT("Validation_BuildSync_NoArtifacts"), TEXT("Validation_BuildSync_ArtifactNotFound")}),
			},
			CreateNamedArtifactSelectionWidget()
		)
		.AddDirectoryString(LOCTEXT("DestinationDirectoryString", "Destination Directory"),
			{
				.GetValue = [this]() { return GetDestinationDirectory(); },
				.SetValue = [this](FString InDestination) { SetDestinationDirectory(InDestination); },
				.IsVisible = [this]() { return GetBuildType() != TEXT("oplog"); }
			},
			false,
			LOCTEXT("DestinationDirectoryTooltip", "Will override the staging directory in BuildCookRun instances")
		)
	;
}


// inspired by SBuildLogin from StorageServerWidgets
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> FBuildSyncLaunchExtensionInstance::CreateBuildStorageConnectionWidget()
{
	using namespace UE::Zen::Build;

	auto GetConnectionStatusText = [this]()
	{
		switch (GetBuildServiceConnectionState())
		{
			case FBuildServiceInstance::EConnectionState::NotStarted:					return LOCTEXT("BuildConnect_StatusValueNotConnected", "Not Connected");
			case FBuildServiceInstance::EConnectionState::ConnectionInProgress:			return LOCTEXT("BuildConnect_StatusValueConnecting", "Connecting...");
			case FBuildServiceInstance::EConnectionState::ConnectionSucceeded:			return FText::Format(LOCTEXT("BuildConnect_StatusValueConnected", "Connected to {0}"), FText::FromStringView(BuildInfo->GetServiceInstance()->GetEffectiveDomain()));
			case FBuildServiceInstance::EConnectionState::ConnectionFailed:				return LOCTEXT("BuildConnect_StatusValueConnectionFailed", "Connection failed");
			default:																	return LOCTEXT("BuildConnect_StatusValueError", "Error");
		}
	};

	auto GetConnectLabel = [this]()
	{
		return (GetBuildServiceConnectionState() == FBuildServiceInstance::EConnectionState::NotStarted) ? LOCTEXT("BuildLogin_ConnectLink", "Click to Connect") : LOCTEXT("BuildLogin_ReconnectLink", "Reconnect");
	};

	return SNew(SHorizontalBox)

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2,4)
		[
			SNew(STextBlock)
			.Text_Lambda(GetConnectionStatusText)
			.Visibility_Lambda( [this]() { return (GetBuildServiceConnectionState() == FBuildServiceInstance::EConnectionState::NotStarted) ? EVisibility::Collapsed : EVisibility::Visible; } )
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2,4)
		[
			SNew(SHyperlink)
			.Text_Lambda(GetConnectLabel)
			.OnNavigate_Lambda( [this]() { BuildInfo->Connect(); } )
			.Visibility_Lambda([this]() { return (GetBuildServiceConnectionState() == FBuildServiceInstance::EConnectionState::NotStarted || GetBuildServiceConnectionState() == FBuildServiceInstance::EConnectionState::ConnectionFailed) ? EVisibility::Visible : EVisibility::Collapsed; } )
		]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> FBuildSyncLaunchExtensionInstance::CreateBuildFilterWidget()
{
	return SNew(SHorizontalBox)

		// search filter box
		+SHorizontalBox::Slot()
		.FillWidth(1)
		.Padding(2,1)
		[
			SNew(SSearchBox)
			.HintText(LOCTEXT("BuildSearchHint", "Find build..."))
			.InitialText_Lambda( [this]()
			{
				return FText::FromString(*FilterText); 
			})
			.OnTextChanged_Lambda( [this](const FText& SearchText)
			{
				FilterText = SearchText.ToString();
				RefreshFilteredBuilds();
			})
		]

		// show preflights
		+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2,1)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("PreflightsToolTip", "Also show preflight builds"))
				.IsChecked_Lambda( [this]() { return bShowPreflights ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
				.OnCheckStateChanged_Lambda( [this](ECheckBoxState State) { bShowPreflights = (State == ECheckBoxState::Checked); RefreshFilteredBuilds(); } )
				.IsFocusable(true)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PreflightsLabel", "Preflights"))
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				]
			]

		// show unsuitable builds
		+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2,1)
			[
				SNew(SCheckBox)
				.ToolTipText(LOCTEXT("UnsuitableToolTip", "Also show builds that may not be suitable"))
				.IsChecked_Lambda( [this]() { return bShowUnsuitable ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
				.OnCheckStateChanged_Lambda( [this](ECheckBoxState State) { bShowUnsuitable = (State == ECheckBoxState::Checked); RefreshFilteredBuilds(); } )

				[
					SNew(STextBlock)
					.Text(LOCTEXT("UnsuitableLabel", "Unsuitable"))
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				]
			]
	;

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> FBuildSyncLaunchExtensionInstance::CreateBuildTypeSelectionWidget()
{
	using namespace UE::Zen::Build;

	auto GetAvailableBuildTypes = [this]()
	{
		TArray<FString> BuildTypes;
		BuildInfo->GetBuildTypesMap().GetKeys(BuildTypes);
		return BuildTypes;
	};

	auto GetDisplayName = [this](const FString& BuildTypeName)
	{
		const TSharedRef<FBuildListRetriever::FBuildType, ESPMode::ThreadSafe>* BuildTypePtr = BuildInfo->GetBuildTypesMap().Find(BuildTypeName);
		if (BuildTypePtr != nullptr)
		{
			return (*BuildTypePtr)->DisplayName;
		}

		return LOCTEXT("UnknownBuildTypeLabel", "Unknown");
	};


	return SNew(SCustomLaunchCombo<FString>)
		.OnSelectionChanged(this, &FBuildSyncLaunchExtensionInstance::SetBuildType)
		.SelectedItem(this, &FBuildSyncLaunchExtensionInstance::GetBuildType)
		.Items_Lambda( [GetAvailableBuildTypes]() { return GetAvailableBuildTypes(); } )
		.GetDisplayName_Lambda(GetDisplayName)
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TArray<FString> FBuildSyncLaunchExtensionInstance::GetAvailableArtifactsForSelectedBuild() const
{
	TArray<FString> AvailableBuilds;
	if (SelectedBuild.IsValid())
	{
		SelectedBuild->Group->NamedArtifacts.GenerateKeyArray(AvailableBuilds);
		AvailableBuilds.Sort();
	}
	return AvailableBuilds;
};

TSharedRef<SWidget> FBuildSyncLaunchExtensionInstance::CreateNamedArtifactSelectionWidget()
{
	auto GetDisplayName = [this](const FString& Value)
	{
		FText Result = FText::FromString(Value);
		if (SelectedBuild.IsValid() && SelectedBuild->Group->NamedArtifacts.Contains(Value))
		{
			return Result;
		}
		else
		{
			return FText::Format(LOCTEXT("NamedArtifactMissingLabel","{0} (not in this build)"), Result);
		}
	};

	auto GetItemIcon = [this]( const FString& Item )
	{
		const UE::Zen::Build::FBuildServiceInstance::FBuildRecord* BuildRecord = SelectedBuild.IsValid() ? SelectedBuild->Group->NamedArtifacts.Find(Item) : nullptr;

		FString Platform = BuildInfo->GetPlatformNameFromNamedArtifact(Item, BuildRecord);

		FName IconName = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(Platform).GetIconStyleName(EPlatformIconSize::Normal);
		if (IconName.IsNone())
		{
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.FilledCircle");
		}
		else
		{

			return FSlateIcon(FAppStyle::GetAppStyleSetName(), IconName);
		}
	};

	return SNew(SCustomLaunchStringCombo)
		.OnSelectedItemsChanged(this, &FBuildSyncLaunchExtensionInstance::SetNamedArtifacts)
		.SelectedItems(this, &FBuildSyncLaunchExtensionInstance::GetNamedArtifacts)
		.Items(this, &FBuildSyncLaunchExtensionInstance::GetAvailableArtifactsForSelectedBuild)
		.GetItemIcon_Lambda(GetItemIcon)
		.GetItemIcon_Lambda(GetItemIcon)
		.GetDisplayName_Lambda(GetDisplayName)
		.ActionType(EUserInterfaceActionType::ToggleButton)
	;
}



UE::Zen::Build::FBuildServiceInstance::EConnectionState FBuildSyncLaunchExtensionInstance::GetBuildServiceConnectionState() const
{
	return BuildInfo->GetServiceInstance()->GetConnectionState();
}



void FBuildSyncLaunchExtensionInstance::OnBuildsRefreshed()
{
	CacheUnsuitableBuilds();

	const TSharedPtr<FBuildInfoHelper::FBuildInfo> OldSelectedBuild = SelectedBuild;

	if (BuildInfo->GetBuildInfos().Num() > 0)
	{
		// we already have a selection & the list was refreshed - try to find the same build in the new list. if we fail, clear the selection
		if (SelectedBuild != nullptr)
		{
			int32 Index = BuildInfo->GetBuildInfos().IndexOfByPredicate( [this]( const TSharedPtr<FBuildInfoHelper::FBuildInfo>& Build )
			{
				return (Build->Group->DisplayName == SelectedBuild->Group->DisplayName);
			});
			SelectedBuild = (Index == -1) ? nullptr : BuildInfo->GetBuildInfos()[Index];
		}

		if (SelectedBuild == nullptr)
		{
			AutoSelectBestBuild();
		}
	}
	else
	{
		SelectedBuild = nullptr;
	}

	BroadcastEvent(TEXT("SelectedBuildChanged"));

	RefreshFilteredBuilds();

	if (CachedNamedArtifacts.IsEmpty())
	{
		AutoSelectNamedArtifacts();
	}
	else
	{
		// Remove artifacts that are not available for the selected build from the cached artifacts
		TArray<FString> AvailableArtifacts = GetAvailableArtifactsForSelectedBuild();
		CachedNamedArtifacts.RemoveAll([&AvailableArtifacts](const FString& Item)
		{
			return !AvailableArtifacts.Contains(Item);
		});

		SetNamedArtifacts(CachedNamedArtifacts);
	}

	GetProfile()->RefreshCustomWarningsAndErrors();
}

void FBuildSyncLaunchExtensionInstance::RefreshFilteredBuilds()
{
	FilteredBuildInfos.Reset();

	for (TSharedPtr<FBuildInfoHelper::FBuildInfo> Build : BuildInfo->GetBuildInfos())
	{		
		// optionally filter out preflights
		if (!bShowPreflights && Build->Group->bIsPreflight)
		{
			continue;
		}

		// optionally filter out unsuitable builds 
		if (!bShowUnsuitable && CachedUnsuitableBuilds.Contains(Build))
		{
			continue;
		}

		// add search filtering
		if (!FilterText.IsEmpty() && !Build->Group->DisplayName.Contains(FilterText))
		{
			continue;
		}

		FilteredBuildInfos.Add(Build);
	}

	if (BuildCombo.IsValid())
	{
		BuildCombo->RefreshOptions();
	}
}



void FBuildSyncLaunchExtensionInstance::SetSelectedBuild(TSharedPtr<FBuildInfoHelper::FBuildInfo> Build)
{
	if (SelectedBuild != Build)
	{
		SelectedBuild = Build;
		GetProfile()->RefreshCustomWarningsAndErrors();
		CacheUnsuitableBuilds();
		BroadcastEvent(TEXT("SelectedBuildChanged"));
	}
}

TSharedPtr<FBuildInfoHelper::FBuildInfo> FBuildSyncLaunchExtensionInstance::GetSelectedBuild() const
{
	return SelectedBuild;
}

bool FBuildSyncLaunchExtensionInstance::GetBuildSuitability( TSharedPtr<FBuildInfoHelper::FBuildInfo> Build, FText* OutReason ) const
{
	if (OutReason == nullptr)
	{
		return !CachedUnsuitableBuilds.Contains(Build);
	}
	else
	{
		const FText* ReasonPtr = CachedUnsuitableBuilds.Find(Build);
		if (ReasonPtr)
		{
			// build is unsuitable - return why
			(*OutReason) = (*ReasonPtr);
			return false;
		}

		// build is suitable
		return true;
	}
}

void FBuildSyncLaunchExtensionInstance::CacheUnsuitableBuilds()
{
	CachedUnsuitableBuilds.Reset();

	for (TSharedPtr<FBuildInfoHelper::FBuildInfo> Build : BuildInfo->GetBuildInfos())
	{
		if (!CachedNamedArtifacts.IsEmpty())
		{
			TArray<FText> MissingNamedArtifacts;
			for (const FString& NamedArtifact : CachedNamedArtifacts)
			{
				if (!Build->Group->NamedArtifacts.Contains(NamedArtifact))
				{
					MissingNamedArtifacts.Add(FText::FromString(*NamedArtifact) );
				}
			}

			if (!MissingNamedArtifacts.IsEmpty())
			{
				FText Separator = LOCTEXT("Separator",", ");
				FText Reason = FText::Format(LOCTEXT("BuildUnsuitable_MissingPlatform", "this build does not contain the selected Build Platform {0}"), FText::Join(Separator, MissingNamedArtifacts) );
				CachedUnsuitableBuilds.Add(Build, Reason);
				continue;
			}
		}


	}
}


void FBuildSyncLaunchExtensionInstance::SetBuildType( FString InBuildType )
{
	SetNamedArtifacts(TArray<FString>());

	SetConfigString(EConfig::PerProfile, TEXT("BuildType"), InBuildType );
	BuildInfo->SetBuildType(InBuildType);
}

FString FBuildSyncLaunchExtensionInstance::GetBuildType() const
{
	FString Result = GetConfigString(EConfig::PerProfile, TEXT("BuildType"));
	if (!Result.IsEmpty())
	{
		return Result;
	}

	if (auto* Pair = BuildInfo->GetBuildTypesMap().FindArbitraryElement())
	{
		return Pair->Key;
	}
	
	return FBuildInfoHelper::DefaultBuildType;
}
void FBuildSyncLaunchExtensionInstance::SetNamedArtifacts( TArray<FString> InNamedArtifacts )
{
	CachedNamedArtifacts = InNamedArtifacts;
	GetProfile()->RefreshCustomWarningsAndErrors();
	CacheUnsuitableBuilds();

	SetConfigString(EConfig::PerProfile, TEXT("NamedArtifacts"), FString::Join(CachedNamedArtifacts,TEXT(";")));
}

TArray<FString> FBuildSyncLaunchExtensionInstance::GetNamedArtifacts() const
{
	return CachedNamedArtifacts;
}





void FBuildSyncLaunchExtensionInstance::SetDestinationDirectory(FString InPath)
{
	if (FPaths::IsRelative(InPath))
	{
		DestinationDirectory = FPaths::ConvertRelativePathToFull(InPath);
	}
	else
	{
		DestinationDirectory = InPath;
	}
	SetConfigString(EConfig::PerProfile, TEXT("DestinationDirectory"), DestinationDirectory);

	TArray<ILauncherProfileBuildCookRunRef> BCRs = GetProfile()->GetBuildCookRunCommands();
	for (auto BuildCookRun : BCRs)
	{
		BuildCookRun->SetPackageDirectory(DestinationDirectory);
	}
}

FString FBuildSyncLaunchExtensionInstance::GetDestinationDirectory() const
{
	return DestinationDirectory;
}

void FBuildSyncLaunchExtensionInstance::AutoSelectBestBuild()
{
	const TArray<const PlatformInfo::FTargetPlatformInfo*> PlatformInfos = GetModel()->GetPlatformInfosForAllCommands(GetProfile());

	auto HasPlatforms = [PlatformInfos]( const TSharedPtr<FBuildInfoHelper::FBuildInfo>& Build, bool bNeedAllPlatforms )
	{
		for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : PlatformInfos)
		{
			const TArray<FString>* ArtifactsPtr = Build->PlatformToArtifacts.Find(PlatformInfo->IniPlatformName);
			if (ArtifactsPtr == nullptr)
			{
				return false;
			}
			if (!bNeedAllPlatforms)
			{
				return true;
			}
		}

		return true;
	};

	for ( int Pass = 0; Pass < 4; Pass++ ) // 0 = !PF + All Platforms   1 = !PF + Any Platform   2 = !PF    3 = anything
	{
		const bool bNeedAllPlatforms = (Pass == 0);

		for (const TSharedPtr<FBuildInfoHelper::FBuildInfo>& Build : BuildInfo->GetBuildInfos())
		{
			if (Pass <= 2 && Build->Group->bIsPreflight)
			{
				continue;
			}

			if (Pass <= 1 && !HasPlatforms(Build, bNeedAllPlatforms))
			{
				continue;
			}

			SelectedBuild = Build;
			break;
		}

		// stop when we find a build
		if (SelectedBuild.IsValid())
		{
			// if we had to loosen the requirements, update the filter to match
			bool bFiltersUpdated = false;
			if (!bShowUnsuitable && Pass > 0)
			{
				bShowUnsuitable = true;
				bFiltersUpdated = true;
			}
			if (!bShowPreflights && Pass > 2)
			{
				bShowPreflights = true;
				bFiltersUpdated = true;
			}

			if (bFiltersUpdated)
			{
				RefreshFilteredBuilds();
			}

			break;
		}
	}
}

void FBuildSyncLaunchExtensionInstance::AutoSelectNamedArtifacts()
{
	if (SelectedBuild.IsValid())
	{
		TArray<FString> NewNamedArtifacts;

		const TArray<const PlatformInfo::FTargetPlatformInfo*> PlatformInfos = GetModel()->GetPlatformInfosForAllCommands(GetProfile());
		for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : PlatformInfos)
		{
			const TArray<FString>* ArtifactsPtr = SelectedBuild->PlatformToArtifacts.Find(PlatformInfo->IniPlatformName);
			if (ArtifactsPtr != nullptr)
			{
				// sort artifacts by length... not an ideal way to find the most likely candidate. expectation is that less common candidates may have extra suffixes..
				TArray<FString> Artifacts = (*ArtifactsPtr);
				Artifacts.Sort( [](const FString& A, const FString& B)
				{
					return A.Len() < B.Len();
				});

				for (const FString& Artifact : Artifacts)
				{
					const UE::Zen::Build::FBuildServiceInstance::FBuildRecord* BuildRecord = SelectedBuild->Group->NamedArtifacts.Find(Artifact);
					if (BuildRecord)
					{
						UE::Zen::Build::FBuildListRetriever::FBucketInfo BucketInfo;
						if (UE::Zen::Build::FBuildListRetriever::GetBucketInfo(BuildRecord->BucketId, BucketInfo))
						{
							if (BucketInfo.BuildType == GetBuildType()) // would be nice to be able to pick the one that matches the profile's Build Target
							{
								NewNamedArtifacts.AddUnique(Artifact);
								break; // only pick one artifact per platform
							}
						}
					}
				}
			}
		}

		if (!NewNamedArtifacts.IsEmpty())
		{
			SetNamedArtifacts(NewNamedArtifacts);
		}
	}
}

TSet<const PlatformInfo::FTargetPlatformInfo*> FBuildSyncLaunchExtensionInstance::GetCookPlatforms() const
{
	TArray<const PlatformInfo::FTargetPlatformInfo*> PlatformInfos = GetModel()->GetPlatformInfosForAllCommands(GetProfile());

	TSet<const PlatformInfo::FTargetPlatformInfo*> Result(PlatformInfos);
	return MoveTemp(Result);
}

void FBuildSyncLaunchExtensionInstance::OnProjectChanged()
{
	CachedUnsuitableBuilds.Reset();
	BuildInfo->SetProjectName(GetProfile()->GetProjectName());
}

void FBuildSyncLaunchExtensionInstance::OnPropertyChanged()
{
	
	// see if the selected platforms have changed
	TSet<const PlatformInfo::FTargetPlatformInfo*> CookPlatforms = GetCookPlatforms();
	bool bPlatformsChanged = CookPlatforms.Num() != CachedCookPlatforms.Num();
	if (!bPlatformsChanged)
	{
		for (const PlatformInfo::FTargetPlatformInfo* Platform : CookPlatforms) // why is there no Compare function for TSet?!
		{
			if (!CachedCookPlatforms.Contains(Platform))
			{
				bPlatformsChanged = true;
				break;
			}
		}
	}
	if (bPlatformsChanged)
	{
		CachedCookPlatforms = CookPlatforms;
		AutoSelectNamedArtifacts();
	}
}



void FBuildSyncLaunchExtensionInstance::OnValidateProfile()
{
	if (GIsEditor)
	{
		GetProfile()->AddCustomError( TEXT("Validation_BuildSync_NotInEditor"), LOCTEXT("Validation_BuildSync_NotInEditor", "The Build Sync extension is not available in the editor because it may update the editor binaries. Use UnrealFrontend instead."));
	}

	if (!BuildInfo->IsConfigured())
	{
		GetProfile()->AddCustomWarning(TEXT("Validation_BuildSync_UnconfiguredBuildInfo"), BuildInfo->GetErrorText());
	}

	if (BuildInfo->IsRefreshing())
	{
		// fixme: disable the launch button but don't show error banner? it just seems distracting during the refresh for the banner to come in and out
	}
	else
	{
		if (!SelectedBuild.IsValid())
		{
			GetProfile()->AddCustomError( TEXT("Validation_BuildSync_NoBuild"), LOCTEXT("Validation_BuildSync_NoBuild", "No build selected"));
		}
		if (CachedNamedArtifacts.IsEmpty())
		{
			GetProfile()->AddCustomError( TEXT("Validation_BuildSync_NoArtifacts"), LOCTEXT("Validation_BuildSync_NoArtifacts", "No build platforms selected"));
		}
		if (!CachedNamedArtifacts.IsEmpty() && SelectedBuild.IsValid())
		{
			bool bHasMissingNamedArtifacts = CachedNamedArtifacts.ContainsByPredicate( [this]( const FString& NamedArtifact )
			{
				return !SelectedBuild->Group->NamedArtifacts.Contains(NamedArtifact);
			});

			if (bHasMissingNamedArtifacts)
			{
				GetProfile()->AddCustomError( TEXT("Validation_BuildSync_ArtifactNotFound"), LOCTEXT("Validation_BuildSync_ArtifactNotFound", "This build does not contain the selected build platforms"));
			}
		}
	}

	if (BuildInfo->GetBuildType() == TEXT("oplog") && SelectedBuild.IsValid())
	{
		ILauncherProfileBuildCookRunPtr BCR = GetProfile()->GetFirstBuildCookRun();

		if (BCR.IsValid() && BCR->IsEnabled() && BCR->IsUsingZenStreaming() && BCR->GetCookMode() == ELauncherProfileCookModes::DoNotCook)
		{
			if (FString* SyncedClStr = GetProfile()->GetCustomStringProperties().Find(TEXT("SyncedChangelist")))
			{
				int32 SyncedCl = FCString::Atoi(**SyncedClStr);
				if (SelectedBuild->Group->Changelist != SyncedCl)
				{
					GetProfile()->AddCustomError( TEXT("Validation_BuildSync_NeedCook"), LOCTEXT("Validation_BuildSync_NeedCook", "The downloaded cook snapshot version is different than the synced project version. Enable cooking or incremental cooking"));
				}
			}
		}
	}
}




void FBuildSyncLaunchExtensionInstance::CustomizeUATCommandLine( FString& InOutCommandLine )
{
	if (!SelectedBuild.IsValid())
	{
		return;
	}

	FString CommandLine;

	CommandLine += FString::Printf(TEXT(" -project=\"%s\" "), *FPaths::ConvertRelativePathToFull(GetProfile()->GetProjectPath()));
	CommandLine += FString::Printf(TEXT(" -CL=%s"), *SelectedBuild->Group->CommitIdentifier);

	FString Branch = FEngineVersion::Current().GetBranch();
	CommandLine += FString::Printf(TEXT(" -Stream=%s"), *Branch);

	CommandLine += FString::Printf(TEXT(" -buildtype=%s"), *GetBuildType());
	CommandLine += FString::Printf(TEXT(" -buildplatform=%s"), *FString::Join(CachedNamedArtifacts, TEXT("+")));

	if (!DestinationDirectory.IsEmpty())
	{
		CommandLine += FString::Printf(TEXT(" -stagingdirectory=\"%s\""), *DestinationDirectory);
	}

	const bool bSkipContent = GetConfigBool(EConfig::PerProfile, TEXT("SkipContentInStaged"));
	if (bSkipContent)
	{
		CommandLine += TEXT(" -SkipContentInStaged");
	}

	InOutCommandLine += CommandLine;
}

void FBuildSyncLaunchExtensionInstance::InternalInitialize()
{
	FLaunchExtensionInstance::InternalInitialize();

	// now the command has been added, start the connection process if we have a project
	if (!GIsEditor && !GetProfile()->GetProjectName().IsEmpty())
	{
		BuildInfo->Connect();
	}
}

void FBuildSyncLaunchExtensionInstance::OnUATCommandAdded( ILauncherProfileUATCommandRef InUATCommand )
{
	InUATCommand->SetUATCommand(TEXT("BuildAcquire")); // RunUAT BuildAcquire [...] 
}

void FBuildSyncLaunchExtensionInstance::OnOtherUATCommandAdded(const ILauncherProfileUATCommandRef& InUATCommand)
{
	ILauncherProfileBuildCookRunPtr BuildCookRun = InUATCommand->AsBuildCookRun();
	if (BuildCookRun.IsValid() && !DestinationDirectory.IsEmpty())
	{
		BuildCookRun->SetPackageDirectory(DestinationDirectory);
	}
}

TArray<FString> FBuildSyncLaunchExtensionInstance::GetSelectedBuildBackends() const
{
	return SelectedBuild ? SelectedBuild->Backends : TArray<FString>();
}










FBuildSyncLaunchExtension::FBuildSyncLaunchExtension()
{
}

TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FBuildSyncLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
{
	return MakeShared<FBuildSyncLaunchExtensionInstance>(InArgs);
}

const TCHAR* FBuildSyncLaunchExtension::GetInternalName() const
{
	return TEXT("BuildSync");
}

FText FBuildSyncLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Build Sync");
}



#undef LOCTEXT_NAMESPACE
