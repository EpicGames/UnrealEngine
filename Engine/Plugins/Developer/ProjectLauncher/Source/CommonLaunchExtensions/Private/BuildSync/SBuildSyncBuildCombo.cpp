// Copyright Epic Games, Inc. All Rights Reserved.

#include "BuildSync/SBuildSyncBuildCombo.h"

#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Math/ColorList.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"


#define LOCTEXT_NAMESPACE "SBuildSyncBuildCombo"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SBuildSyncBuildCombo::Construct(const FArguments& InArgs)
{
	OptionsSource = InArgs._OptionsSource;
	OnSelectionChanged = InArgs._OnSelectionChanged;
	GetItemSuitability = InArgs._GetItemSuitability;

	SelectedItem = InArgs._SelectedItem;
	RequiredPlatforms = InArgs._RequiredPlatforms;

	check (InArgs._FilterWidget.IsValid());

	ChildSlot
	[
		SAssignNew(ComboButton, SComboButton)
		.IsFocusable(true)
		.CollapseMenuOnParentFocus(true)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SBuildSyncBuildCombo::GetSelectedBuildInfoName)
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
		]
		.MenuContent()
		[
			SNew(SBox)
			.MaxDesiredHeight(512)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(8)
				[
					InArgs._FilterWidget.ToSharedRef()
				]

				+SVerticalBox::Slot()
				.FillHeight(1)
				.Padding(8)
				[
					SAssignNew(BuildInfosListView, SListView<TSharedPtr<FBuildInfoHelper::FBuildInfo>>)
					.ListItemsSource(OptionsSource)
					.OnGenerateRow( this, &SBuildSyncBuildCombo::OnGenerateBuildInfoRow )
					.OnSelectionChanged( this, &SBuildSyncBuildCombo::OnBuildInfoSelectionChanged )
					.SelectionMode(ESelectionMode::SingleToggle)
					.IsFocusable(true)
					.OnKeyDownHandler_Lambda( [this](const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
					{
						if (InKeyEvent.GetKey() == EKeys::Enter && BuildInfosListView->GetNumItemsSelected() > 0)
						{
							OnBuildInfoSelectionChanged( BuildInfosListView->GetSelectedItems()[0], ESelectInfo::OnKeyPress);
							return FReply::Handled();
						}

						return FReply::Unhandled();
					})
				]
			]
		]
		.OnMenuOpenChanged_Lambda( [this](bool bOpen)
		{
			if (bOpen)
			{
				if (SelectedItem.Get().IsValid())
				{
					BuildInfosListView->RequestScrollIntoView(SelectedItem.Get());
					FSlateApplication::Get().SetKeyboardFocus(BuildInfosListView, EFocusCause::SetDirectly);
				}
			}
			else
			{
				FSlateApplication::Get().SetKeyboardFocus(ComboButton, EFocusCause::SetDirectly);
			}
		})
	];

	ComboButton->SetMenuContentWidgetToFocus(BuildInfosListView);

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<ITableRow> SBuildSyncBuildCombo::OnGenerateBuildInfoRow( TSharedPtr<FBuildInfoHelper::FBuildInfo> BuildInfo, const TSharedRef<STableViewBase>& OwnerTable) const
{
	if (!BuildInfo.IsValid())
	{
		return SNew(STableRow<TSharedPtr<FBuildInfoHelper::FBuildInfo>>, OwnerTable)
		[
			SNullWidget::NullWidget
		];
	}
	
	static const FColor BadgeColors[] =
	{
		FColorList::LightBlue,			// Starting
		FColorList::OrangeRed,			// Failure
		FColorList::Orange,				// Warning
		FColorList::MediumSpringGreen,	// Success
		FColorList::LightGrey,			// Skipped
		FColorList::LightGrey,			// Unknown

	};
	static_assert(UE_ARRAY_COUNT(BadgeColors) == (int)FUGSBuildInfoRetriever::FUGSBadge::EState::MAX);


	auto GetSuitablilityColor = [this, BuildInfo]()
	{
		bool bIsSuitable = !GetItemSuitability.IsBound() || GetItemSuitability.Execute(BuildInfo, nullptr);
		return bIsSuitable ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
	};

	auto GetSuitabilityToolTip = [this, BuildInfo]()
	{
		FText Text = FText::GetEmpty();
		if (GetItemSuitability.IsBound())
		{
			GetItemSuitability.Execute(BuildInfo, &Text);
		}

		return Text;
	};


	// extract the build suffix - anything after ++Foo+Bar-CL-123
	FString BuildVersionSuffix;
	FString Suffix = BuildInfo->Group->Suffix;
	if (Suffix.IsEmpty())
	{
		int32 Idx = BuildInfo->Group->DisplayName.Find(BuildInfo->Group->CommitIdentifier);
		if (Idx != -1)
		{
			Suffix = BuildInfo->Group->DisplayName.RightChop(Idx + BuildInfo->Group->CommitIdentifier.Len() + 1);
		}
	}
	if (!Suffix.IsEmpty())
	{
		BuildVersionSuffix = TEXT("[") + Suffix + TEXT("]");
	}

	// prepare the first row
	TSharedPtr<SHorizontalBox> BuildInfoBox;
	TSharedPtr<SVerticalBox> BuildInfoInfoBox;

	TSharedRef<SBox> Result = 
		SNew(SBox)
		.WidthOverride(512) // maintain everything at the same width
		.Padding(4,8)
		[
			SNew(SHorizontalBox)

			// check mark for selected item
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Check"))
				.Visibility_Lambda( [this, BuildInfo]() { return (SelectedItem.Get() == BuildInfo) ? EVisibility::Visible : EVisibility::Hidden; } )
			]

			// item
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SAssignNew(BuildInfoInfoBox, SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(BuildInfoBox, SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()

					// changelist number
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(1)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::FromString(*BuildInfo->Group->CommitIdentifier))
						.Font(FCoreStyle::Get().GetFontStyle("NormalFontBold"))
						.ColorAndOpacity_Lambda(GetSuitablilityColor)
					]

					// additional detail (preflight code etc)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8,1)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Visibility(BuildVersionSuffix.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
						.Text(FText::FromString(*BuildVersionSuffix))
						.Font(FCoreStyle::Get().GetFontStyle("NormalFont"))
						.ColorAndOpacity_Lambda(GetSuitablilityColor)
					]

					// age
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(8,1)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(GetAge(BuildInfo->Group->CreatedAt))
						.Font(FCoreStyle::Get().GetFontStyle("NormalFontItalic"))
						.ColorAndOpacity_Lambda(GetSuitablilityColor)
					]
				]
			]
		]
	;


	// platform icons & artifacts. use a plain circle for platforms we don't know about
	for (const FName& Platform : BuildInfo->Platforms)
	{
		FName IconName = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(Platform).GetIconStyleName(EPlatformIconSize::Normal);

		const FSlateBrush* Brush = IconName.IsNone() ? FAppStyle::Get().GetBrush("Icons.FilledCircle") : FAppStyle::Get().GetBrush(IconName);
		const FSlateColor& Color = IconName.IsNone() ? FSlateColor::UseSubduedForeground() : GetSuitablilityColor();

		BuildInfoBox->AddSlot()
			.AutoWidth()
			.Padding(1)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(Brush)
				.ToolTipText(FText::FromString(FString::Join(BuildInfo->PlatformToArtifacts[Platform], TEXT("\n"))))
				.ColorAndOpacity(Color)
				.DesiredSizeOverride(FVector2D(12,12))
			]
		;
	}




	// add the UGS build information if we have it
	if (BuildInfo->UGSBuildInfo.IsValid())
	{
		// success / failure badge (not using this at the moment - setting the user count text color instead)
		//if (BuildInfo->UGSBuildInfo->NumSuccess > 0 || BuildInfo->UGSBuildInfo->NumFailed > 0)
		//{
		//	bool bSuccess = (BuildInfo->UGSBuildInfo->NumSuccess > BuildInfo->UGSBuildInfo->NumFailed);
		//
		//	BuildInfoBox->InsertSlot(0)
		//		.AutoWidth()
		//		.Padding(1)
		//		[
		//			SNew(SImage)
		//			.Image(FAppStyle::Get().GetBrush("Icons.FilledCircle"))
		//			.ColorAndOpacity( bSuccess ? FColorList::MediumSpringGreen : FColorList::OrangeRed )
		//		]
		//	;
		//}




		// user count
		if (BuildInfo->UGSBuildInfo->NumUsers > 0)
		{
			FSlateColor Color = FSlateColor::UseForeground();
			if (BuildInfo->UGSBuildInfo->NumSuccess > 0 || BuildInfo->UGSBuildInfo->NumFailed > 0)
			{
				bool bSuccess = (BuildInfo->UGSBuildInfo->NumSuccess > BuildInfo->UGSBuildInfo->NumFailed);
				Color = bSuccess ? FColorList::MediumSpringGreen : FColorList::OrangeRed;
			}

			BuildInfoBox->AddSlot()
				.FillWidth(1)
				.Padding(8,1)
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("NumUsersLabel", "{0} users"), BuildInfo->UGSBuildInfo->NumUsers ))
					.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					.ColorAndOpacity(Color)
				]
			;
		}
	

		// add UGS badges array if there's anything to display
		if (BuildInfo->UGSBuildInfo->Badges.Num() > 0)
		{
			TSharedRef<SHorizontalBox> BadgesBox = SNew(SHorizontalBox);
			BuildInfoInfoBox->AddSlot()
				.Padding(8,1)
				[
					SNew(SBox)
					.HeightOverride(14)
					[
						BadgesBox
					]
				]
			;

			for ( const FUGSBuildInfoRetriever::FUGSBadge& Badge : BuildInfo->UGSBuildInfo->Badges)
			{
				static bool bShowBadgesWithNoUrl = false; // todo: could make this public
				if (!bShowBadgesWithNoUrl && Badge.URL.IsEmpty())
				{
					continue;
				}

				auto OnClick = [URL = Badge.URL](const FGeometry& InGeometry, const FPointerEvent& InEvent)
				{
					if (FSlateApplication::Get().GetModifierKeys().IsControlDown())
					{
						FPlatformProcess::LaunchURL(*URL, nullptr, nullptr); 
						return FReply::Handled(); 
					}

					return FReply::Unhandled(); 
				};

				auto GetCursor = []()
				{
					return FSlateApplication::Get().GetModifierKeys().IsControlDown() ? EMouseCursor::Hand : EMouseCursor::Default;
				};

				auto GetColor = [this, State = Badge.State, BuildInfo]()
				{
					FLinearColor Color = BadgeColors[(int8)State];
					if (GetItemSuitability.IsBound() && !GetItemSuitability.Execute(BuildInfo, nullptr))
					{
						Color = Color.Desaturate(0.25f);
					}

					return Color;
				};


				FText ToolTipText = FText::Format(LOCTEXT("UgsBadgeTip", "{0}\nCtrl+Click : {1}"), FText::FromString(*Badge.Name), FText::FromString(*Badge.URL) );
				BadgesBox->AddSlot()
					.FillWidth(1)
					.Padding(1,0,0,0)
					.VAlign(VAlign_Fill)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("NoBorder"))
						.Padding(FMargin(0,4))
						.Cursor_Lambda(GetCursor)
						.OnMouseButtonDown_Lambda(OnClick)
						.ToolTipText(ToolTipText)
						[
							SNew(SColorBlock)
							.Color_Lambda(GetColor)
						]
					]
				;
			}
		}
	}



	// add backends display row, if there's anything to display
	TSharedPtr<SHorizontalBox> BackendsBox;
	if (BuildInfo->Backends.Num() > 0)
	{
		BackendsBox = SNew(SHorizontalBox);
		BuildInfoInfoBox->AddSlot()
			.AutoHeight()
			.Padding(8,1)
			[
				BackendsBox.ToSharedRef()
			]
		;

		for (const FString& Backend : BuildInfo->Backends)
		{
			BackendsBox->AddSlot()
				.AutoWidth()
				.Padding(4)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Right)
				[
					SNew(SBorder)
					.Padding(2,4)
					.BorderImage(FCoreStyle::Get().GetBrush("FloatingBorder"))
					.BorderBackgroundColor(FLinearColor::Black)
					[
						SNew(STextBlock)
						.Text(FText::FromString(*Backend))
						.ColorAndOpacity(FLinearColor::White)
						.Justification(ETextJustify::Center)
						.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
					]
				]
			;
		}
	}

	return SNew(STableRow<TSharedPtr<FBuildInfoHelper::FBuildInfo>>, OwnerTable)
	.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
    .ShowSelection(true)
	.ToolTipText_Lambda(GetSuitabilityToolTip)
	[
		Result
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION






FText SBuildSyncBuildCombo::GetAge( const FDateTime& DateTime ) const
{
	FTimespan Age = FDateTime::Now() - DateTime;
	if (Age.GetDays() > 0)
	{
		return FText::Format(LOCTEXT("Timespan_DaysHoursMin", "({0}days {1}hrs {2}mins old)"), Age.GetDays(), Age.GetHours(), Age.GetMinutes() );
	}
	else if (Age.GetHours() > 0)
	{
		return FText::Format(LOCTEXT("Timespan_HoursMin", "({0}hrs {1}mins old)"), Age.GetHours(), Age.GetMinutes() );
	}
	else
	{
		return FText::Format(LOCTEXT("Timespan_Min", "({0}mins old)"), Age.GetMinutes() );
	}
}




void SBuildSyncBuildCombo::OnBuildInfoSelectionChanged( TSharedPtr<FBuildInfoHelper::FBuildInfo> BuildInfo, ESelectInfo::Type InSelectInfo )
{
	if (InSelectInfo != ESelectInfo::Direct)
	{
		// using ESelecitonMode::SingleToggle so the combo box can close even if you select the same item - so need to ignore the 'unselect'
		if (BuildInfo.IsValid()) 
		{
			OnSelectionChanged.ExecuteIfBound(BuildInfo);
		}

		if (InSelectInfo == ESelectInfo::OnKeyPress || InSelectInfo == ESelectInfo::OnMouseClick)
		{
			ComboButton->SetIsOpen( false );
		}
	}
}


FText SBuildSyncBuildCombo::GetSelectedBuildInfoName() const
{
	TSharedPtr<FBuildInfoHelper::FBuildInfo> SelectedBuildInfo = SelectedItem.Get();
	return SelectedBuildInfo.IsValid() ? FText::FromString(*SelectedBuildInfo->Group->DisplayName) : LOCTEXT("NoBuild", "(no build)");
}





void SBuildSyncBuildCombo::RefreshOptions()
{
	BuildInfosListView->RequestListRefresh();

	BuildInfosListView->ClearSelection();
	if (SelectedItem.Get().IsValid())
	{
		BuildInfosListView->SetSelection(SelectedItem.Get());
		BuildInfosListView->RequestScrollIntoView(SelectedItem.Get());
	}
}


#undef LOCTEXT_NAMESPACE
