// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "ProjectLauncherModule.h"
#include "Globals/GlobalsLaunchExtension.h"
#include "Insights/InsightsLaunchExtension.h"
#include "BootTest/BootTestLaunchExtension.h"
#include "AdvancedCook/AdvancedCookLaunchExtension.h"
#include "ProfileWizard/ProfileWizardLaunchExtension.h"
#include "DeprecatedProperties/DeprecatedPropertiesLaunchExtension.h"
#include "BuildSync/BuildSyncLaunchExtension.h"
#include "UgsSync/UgsSyncLaunchExtension.h"
#include "UserUATArgs/UserUATArgsLaunchExtension.h"
#include "UserUATCommand/UserUATCommandLaunchExtension.h"

class FCommonLaunchExtensionsModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
		Extensions.Add( MakeShared<FGlobalsLaunchExtension>() );
		Extensions.Add( MakeShared<FInsightsLaunchExtension>() );
		Extensions.Add( MakeShared<FBootTestLaunchExtension>() );
		Extensions.Add( MakeShared<FAdvancedCookLaunchExtension>() );
		Extensions.Add( MakeShared<FProfileWizardLaunchExtension>() );
		Extensions.Add( MakeShared<FDeprecatedPropertiesLaunchExtension>() );
		Extensions.Add( MakeShared<FBuildSyncLaunchExtension>() );
		Extensions.Add( MakeShared<FUGSSyncLaunchExtension>() );
		Extensions.Add( MakeShared<FUserUATArgsLaunchExtension>() );
		Extensions.Add( MakeShared<FUserUATCommandLaunchExtension>() );

		for (TSharedRef<ProjectLauncher::FLaunchExtension> Extension : Extensions)
		{
			IProjectLauncherModule::Get().RegisterExtension(Extension);
		}
	}

	virtual void ShutdownModule() override
	{
		if (IProjectLauncherModule* ProjectLauncher = IProjectLauncherModule::TryGet())
		{
			for (TSharedRef<ProjectLauncher::FLaunchExtension> Extension : Extensions)
			{
				ProjectLauncher->UnregisterExtension(Extension);
			}
		}

		Extensions.Reset();
	}

private:
	TArray<TSharedRef<ProjectLauncher::FLaunchExtension>> Extensions;
};


IMPLEMENT_MODULE(FCommonLaunchExtensionsModule, CommonLaunchExtensions);

