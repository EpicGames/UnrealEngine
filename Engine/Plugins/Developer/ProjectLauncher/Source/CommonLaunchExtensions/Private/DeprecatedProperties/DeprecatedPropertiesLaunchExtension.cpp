// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeprecatedProperties/DeprecatedPropertiesLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"
#include "HAL/PlatformMisc.h"
#include "Misc/CoreMisc.h"
#include "Widgets/Shared/SCustomLaunchCombo.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBorder.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "FDeprecatedPropertiesLaunchExtensionInstance"



FDeprecatedPropertiesLaunchExtensionInstance::FDeprecatedPropertiesLaunchExtensionInstance( FArgs& InArgs ) 
	: ProjectLauncher::FBuildCookRunCommandExtensionInstance(InArgs)
	, Owner(StaticCastSharedRef<FDeprecatedPropertiesLaunchExtension>(InArgs.Extension))
{
}


TSharedRef<ProjectLauncher::FBuildCookRunExtension> FDeprecatedPropertiesLaunchExtensionInstance::CreateBuildCookRunExtension( const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs )
{
	return MakeShared<FBuildCookRunInstance>(InArgs);
}

bool FDeprecatedPropertiesLaunchExtensionInstance::IsBuildCookRunExtensionEnabledByDefault( const ILauncherProfileBuildCookRunRef& InBuildCookRun ) const
{
	return Owner->HasDeprecatedProperties(InBuildCookRun, GetModel());
}

bool FDeprecatedPropertiesLaunchExtensionInstance::CanToggleBuildCookRunExtension( const ILauncherProfileBuildCookRunRef& InBuildCookRun, bool bWantToEnable ) const
{
	if (!bWantToEnable && Owner->HasDeprecatedProperties(InBuildCookRun, GetModel()))
	{
		return false;
	}
	return true;
}


FDeprecatedPropertiesLaunchExtensionInstance::FBuildCookRunInstance::FBuildCookRunInstance( const FArgs& InArgs )
	: ProjectLauncher::FBuildCookRunExtension( InArgs )
	, Owner(StaticCastSharedRef<FDeprecatedPropertiesLaunchExtension>(InArgs.Extension))
{
}

void FDeprecatedPropertiesLaunchExtensionInstance::FBuildCookRunInstance::CustomizeTree( ProjectLauncher::FLaunchProfileTreeNode& ProfileTreeNode )
{
	AddDefaultHeading(ProfileTreeNode)
		.AddWidget( FText::GetEmpty(),
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.LightGroupBorder"))
			.BorderBackgroundColor_Lambda( [this]()
			{
				return Owner->HasDeprecatedProperties(GetBuildCookRun(), GetModel())
					? FAppStyle::Get().GetSlateColor("Colors.Warning")
					: FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
			})
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.Text_Lambda( [this]() 
				{ 
					return Owner->HasDeprecatedProperties(GetBuildCookRun(), GetModel())
						? LOCTEXT("DeprecatedItemsBanner", "These properties may be removed in future engine releases.") 
						: LOCTEXT("NoDeprecatedItemsBanner", "These properties may be removed in future engine releases. You are not using any at the moment.");
				})
			]
		)
		
		// ... http chunks ...
		.AddBoolean( LOCTEXT("CreateHttpChunksLabel", "Create Http Chunk Install Data"),
			{
				.GetValue = [this]()				{ return GetBuildCookRun()->IsGenerateHttpChunkData(); },
				.SetValue = [this](bool bValue)		{ GetBuildCookRun()->SetGenerateHttpChunkData(bValue); },
				.GetDefaultValue = [this]()			{ return GetDefaultBuildCookRun()->IsGenerateHttpChunkData(); },
			}
		)
		.AddDirectoryString( LOCTEXT("HttpChunkDirLabel", "Http Chunk Install Data Path"),
			{
				.GetValue = [this]()				{ return GetBuildCookRun()->GetHttpChunkDataDirectory(); },
				.SetValue = [this](FString Value)	{ GetBuildCookRun()->SetHttpChunkDataDirectory(Value); },
				.GetDefaultValue = [this]()			{ return GetDefaultBuildCookRun()->GetHttpChunkDataDirectory(); },
				.IsEnabled = [this]()				{ return GetBuildCookRun()->IsGenerateHttpChunkData(); },
			}
		)
		.AddString( LOCTEXT("HttpChunkReleaseName", "Http Chunk Install Release Name"),
			{
				.GetValue = [this]()				{ return GetBuildCookRun()->GetHttpChunkDataReleaseName(); },
				.SetValue = [this](FString Value)	{ GetBuildCookRun()->SetHttpChunkDataReleaseName(Value); },
				.GetDefaultValue = [this]()			{ return GetDefaultBuildCookRun()->GetHttpChunkDataReleaseName(); },
				.IsEnabled = [this]()				{ return GetBuildCookRun()->IsGenerateHttpChunkData(); },
			}
		)

		// ... misc items ...
		.AddBoolean( LOCTEXT("SkipEditorContentLabel", "Don't include editor content in the build"),
			{
				.GetValue = [this]()				{ return GetBuildCookRun()->GetSkipCookingEditorContent(); },
				.SetValue = [this](bool bValue)		{ GetBuildCookRun()->SetSkipCookingEditorContent(bValue); },
				.GetDefaultValue = [this]()			{ return GetDefaultBuildCookRun()->GetSkipCookingEditorContent(); },
			}
		)
		.AddWidget( LOCTEXT("CookerConfigLabel", "Cooker build configuration"), 
			{
				.IsDefault = [this]()				{ return GetBuildCookRun()->GetCookConfiguration() == GetDefaultBuildCookRun()->GetCookConfiguration(); },
				.SetToDefault = [this]()			{ return GetBuildCookRun()->SetCookConfiguration( GetDefaultBuildCookRun()->GetCookConfiguration() ); },
			},
			SNew(SCustomLaunchLexToStringCombo<EBuildConfiguration>)
			.OnSelectionChanged_Lambda( [this](EBuildConfiguration Value) { GetBuildCookRun()->SetCookConfiguration(Value); } )
			.SelectedItem_Lambda( [this]() { return GetBuildCookRun()->GetCookConfiguration(); } )
			.Items(TArray<EBuildConfiguration>{ EBuildConfiguration::Debug, EBuildConfiguration::DebugGame, EBuildConfiguration::Development, EBuildConfiguration::Test, EBuildConfiguration::Shipping })
		)
		.AddBoolean( LOCTEXT("IoStoreLabel", "Use container files for optimized loading (I/O Store)"),
			{
				.GetValue = [this]()				{ return GetBuildCookRun()->IsUsingIoStore(); },
				.SetValue = [this](bool bValue)		{ GetBuildCookRun()->SetUseIoStore(bValue); },
				.GetDefaultValue = [this]()			{ return GetDefaultBuildCookRun()->IsUsingIoStore(); },
			}
		)
		.AddFileString( LOCTEXT("IoStoreRefContainerLabel", "Optional I/O Store Reference Chunk Database"),
			{
				.GetValue = [this]()				{ return GetBuildCookRun()->GetReferenceContainerGlobalFileName(); },
				.SetValue = [this](FString Value)	{ GetBuildCookRun()->SetReferenceContainerGlobalFileName(Value); },
				.GetDefaultValue = [this]()			{ return GetDefaultBuildCookRun()->GetReferenceContainerGlobalFileName(); },
				.IsEnabled = [this]()				{ return GetBuildCookRun()->IsUsingIoStore(); },
			},
			TEXT("global.utoc files|global.utoc")
		)
		.AddFileString( LOCTEXT("IoStoreRefCryptoKeys", "Optional I/O Store Reference Chunk Crypto Key"),
			{
				.GetValue = [this]()				{ return GetBuildCookRun()->GetReferenceContainerCryptoKeysFileName(); },
				.SetValue = [this](FString Value)	{ GetBuildCookRun()->SetReferenceContainerCryptoKeysFileName(Value); },
				.GetDefaultValue = [this]()			{ return GetDefaultBuildCookRun()->GetReferenceContainerCryptoKeysFileName(); },
				.IsEnabled = [this]()				{ return GetBuildCookRun()->IsUsingIoStore(); },
			},
			TEXT("crypto.json files|crypto.json")
		)
	;
}

void FDeprecatedPropertiesLaunchExtension::MakeCustomExtensionSubmenu(FMenuBuilder& MenuBuilder, ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel)
{
	// putting this here for now because it isn't serialized with the profile. @todo: should have a better home!
	MenuBuilder.AddMenuEntry(
		LOCTEXT("BuildUATLabel", "Build UAT"),
		LOCTEXT("BuildUATToolTip", "Build UAT before launching"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this,InProfile]() { InProfile->SetBuildUAT(!InProfile->IsBuildingUAT()); } ),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda( [this,InProfile]() { return InProfile->IsBuildingUAT(); } )
		),
		NAME_None,
		EUserInterfaceActionType::Check
	);
}

void FDeprecatedPropertiesLaunchExtension::GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const
{
	MenuEntry = FExtensionsMenuEntry::Deprecated;	
}



TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FDeprecatedPropertiesLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
{
	return MakeShared<FDeprecatedPropertiesLaunchExtensionInstance>(InArgs);
}

const TCHAR* FDeprecatedPropertiesLaunchExtension::GetInternalName() const
{
	return TEXT("DeprecatedProperties");
}

FText FDeprecatedPropertiesLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Deprecated Properties");
}


bool FDeprecatedPropertiesLaunchExtension::IsCreatedByDefault( ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel ) const
{
	return HasDeprecatedProperties(InProfile, InModel);
}

bool FDeprecatedPropertiesLaunchExtension::HasDeprecatedProperties(const ILauncherProfileBuildCookRunRef& InBuildCookRun, TSharedRef<ProjectLauncher::FModel> InModel) const
{
	const ILauncherProfileBuildCookRunRef& DefaultBuildCookRun = InModel->GetCustomDefaultBuildCookRun();

	return	InBuildCookRun->GetSkipCookingEditorContent()				!=	DefaultBuildCookRun->GetSkipCookingEditorContent() ||
			InBuildCookRun->GetCookConfiguration()						!=	DefaultBuildCookRun->GetCookConfiguration() ||
			InBuildCookRun->IsUsingIoStore()							!=	DefaultBuildCookRun->IsUsingIoStore() ||
			InBuildCookRun->GetReferenceContainerGlobalFileName()		!=	DefaultBuildCookRun->GetReferenceContainerGlobalFileName() ||
			InBuildCookRun->GetReferenceContainerCryptoKeysFileName()	!=	DefaultBuildCookRun->GetReferenceContainerCryptoKeysFileName()
		;
}

bool FDeprecatedPropertiesLaunchExtension::HasDeprecatedProperties(ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel) const
{
	for (const ILauncherProfileBuildCookRunRef& BuildCookRun : InProfile->GetBuildCookRunCommands())
	{
		if (HasDeprecatedProperties(BuildCookRun, InModel))
		{
			return true;
		}
	}

	return false;
}



#undef LOCTEXT_NAMESPACE
