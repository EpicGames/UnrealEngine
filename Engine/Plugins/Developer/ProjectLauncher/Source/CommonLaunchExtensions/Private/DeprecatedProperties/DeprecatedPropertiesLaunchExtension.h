// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/BuildCookRunCommandExtension.h"


class FDeprecatedPropertiesLaunchExtensionInstance : public ProjectLauncher::FBuildCookRunCommandExtensionInstance
{
public:
	FDeprecatedPropertiesLaunchExtensionInstance( FArgs& InArgs );
	virtual ~FDeprecatedPropertiesLaunchExtensionInstance() = default;

	virtual TSharedRef<ProjectLauncher::FBuildCookRunExtension> CreateBuildCookRunExtension( const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs ) override;
	virtual bool IsBuildCookRunExtensionEnabledByDefault( const ILauncherProfileBuildCookRunRef& InBuildCookRun ) const override;
	virtual bool CanToggleBuildCookRunExtension( const ILauncherProfileBuildCookRunRef& InBuildCookRun, bool bWantToEnable ) const override;

protected:
	TSharedRef<class FDeprecatedPropertiesLaunchExtension> Owner;
	class FBuildCookRunInstance : public ProjectLauncher::FBuildCookRunExtension
	{
	public:
		FBuildCookRunInstance(const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs );
		virtual void CustomizeTree( ProjectLauncher::FLaunchProfileTreeNode& ProfileTreeNode ) override;
		TSharedRef<class FDeprecatedPropertiesLaunchExtension> Owner;
	};

};


class FDeprecatedPropertiesLaunchExtension : public ProjectLauncher::FBuildCookRunCommandExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
	virtual bool IsCreatedByDefault( ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel ) const override;
	virtual void GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const override;
	virtual void MakeCustomExtensionSubmenu(FMenuBuilder& MenuBuilder, ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel) override;
	
	bool HasDeprecatedProperties(const ILauncherProfileBuildCookRunRef& InBuildCookRun, TSharedRef<ProjectLauncher::FModel> InModel) const;
	bool HasDeprecatedProperties(ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel) const;
};