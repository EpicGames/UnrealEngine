// Copyright Epic Games, Inc. All Rights Reserved.

#include "Globals/GlobalsLaunchExtension.h"
#include "SocketSubsystem.h"
#include "Misc/OutputDeviceRedirector.h"
#include "Trace/Trace.h"

#define LOCTEXT_NAMESPACE "FGlobalsLaunchExtensionInstance"


const TCHAR* FGlobalsLaunchExtensionInstance::LocalHostVariable     = TEXT("$(LocalHost)");
const TCHAR* FGlobalsLaunchExtensionInstance::LocalHostsVariable    = TEXT("$(LocalHosts)");
const TCHAR* FGlobalsLaunchExtensionInstance::ProjectNameVariable   = TEXT("$(ProjectName)");
const TCHAR* FGlobalsLaunchExtensionInstance::ProjectPathVariable   = TEXT("$(ProjectPath)");

bool FGlobalsLaunchExtensionInstance::GetExtensionVariables( TArray<FString>& OutVariables ) const
{
	OutVariables.Add(LocalHostVariable);
	OutVariables.Add(LocalHostsVariable);
	OutVariables.Add(ProjectNameVariable);
	OutVariables.Add(ProjectPathVariable);
	return true;
}

bool FGlobalsLaunchExtensionInstance::GetExtensionVariableValue( const FString& InVariable, FString& OutValue ) const
{
	if (InVariable == LocalHostVariable || InVariable == LocalHostsVariable)
	{
		bool bSingle = (InVariable == LocalHostVariable);

		TArray<TSharedPtr<FInternetAddr>> AdapterAddresses;
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(AdapterAddresses);
		if (AdapterAddresses.Num() > 0)
		{
			TArray<FString> LocalHosts;
			for (TSharedPtr<FInternetAddr>& LocalHostAddr : AdapterAddresses)
			{
				if (LocalHostAddr.IsValid())
				{
					constexpr bool bAppendPort = false;
					FString AddressString = LocalHostAddr->ToString(bAppendPort);
					if (bSingle) // @todo: any better heuristic than "first" ?
					{
						OutValue = AddressString;
						return true;
					}
					else
					{
						LocalHosts.Add(AddressString);
					}
				}
			}

			if (LocalHosts.Num() > 0)
			{
				OutValue = FString::Join(LocalHosts, TEXT("+"));
				return true;
			}
		}

		OutValue = TEXT("localhost");
		return true;
	}

	else if (InVariable == ProjectNameVariable)
	{
		OutValue = GetProfile()->GetProjectName();
		return true;
	}
	else if (InVariable == ProjectPathVariable)
	{
		OutValue = GetProfile()->GetProjectPath();
		return true;
	}

	return false;
}



TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FGlobalsLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
{
	return MakeShared<FGlobalsLaunchExtensionInstance>(InArgs);
}

const TCHAR* FGlobalsLaunchExtension::GetInternalName() const
{
	return TEXT("Globals");
}

FText FGlobalsLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Globals");
}


#undef LOCTEXT_NAMESPACE
