// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/LaunchExtension.h"

class FGlobalsLaunchExtensionInstance : public ProjectLauncher::FLaunchExtensionInstance
{
public:
	FGlobalsLaunchExtensionInstance( FArgs& InArgs ) : ProjectLauncher::FLaunchExtensionInstance(InArgs) {}
	virtual ~FGlobalsLaunchExtensionInstance() = default;

	virtual bool GetExtensionVariables( TArray<FString>& OutVariables ) const override;
	virtual bool GetExtensionVariableValue( const FString& InVariable, FString& OutValue ) const override;	

private:

	static const TCHAR* LocalHostVariable;
	static const TCHAR* LocalHostsVariable;
	static const TCHAR* ProjectNameVariable;
	static const TCHAR* ProjectPathVariable;
	static const TCHAR* TargetNameVariable;
	static const TCHAR* PlatformNameVariable;
	static const TCHAR* ConfigurationVariable;	
};

class FGlobalsLaunchExtension : public ProjectLauncher::FLaunchExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
	virtual bool IsAlwaysCreated(ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel) const { return true; }
};