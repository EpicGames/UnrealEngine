// Copyright Epic Games, Inc. All Rights Reserved.

#include "Insights/InsightsLaunchExtension.h"
#include "Utils/LauncherCmdLineUtils.h"
#include "Trace/Trace.h"


#define LOCTEXT_NAMESPACE "FInsightsLaunchExtensionInstance"

const TCHAR* FInsightsLaunchExtensionInstance::FileParam = TEXT("-tracefile");
const TCHAR* FInsightsLaunchExtensionInstance::HostParam = TEXT("-tracehost=$(LocalHost)");
const TCHAR* FInsightsLaunchExtensionInstance::TraceParam = TEXT("-trace=");
const TCHAR* FInsightsLaunchExtensionInstance::StatNamedEventsParam = TEXT("-statnamedevents");


TSharedRef<ProjectLauncher::FCmdLineParametersExtension> FInsightsLaunchExtensionInstance::Create( const ProjectLauncher::FCmdLineParametersExtension::FArgs& InArgs )
{
	return MakeShared<FInsightsCommandLineExtension>(InArgs);
}




FInsightsLaunchExtensionInstance::FInsightsCommandLineExtension::FInsightsCommandLineExtension( const ProjectLauncher::FCmdLineParametersExtension::FArgs& InArgs )
	: ProjectLauncher::FCmdLineParametersExtension(InArgs)
{
}

void FInsightsLaunchExtensionInstance::GetCmdLineParameters( TArray<FString>& OutParameters ) const
{
	OutParameters.Add(HostParam);
	OutParameters.Add(FileParam);
	OutParameters.Add(StatNamedEventsParam);
}

FText FInsightsLaunchExtensionInstance::GetCmdLineParameterDisplayName( const FString& InParameter ) const
{
	if (InParameter == HostParam)
	{
		return LOCTEXT("TraceHostLabel", "Trace to a computer");
	}
	else if (InParameter == FileParam)
	{
		return LOCTEXT("TraceFileLabel", "Trace to a file");
	}
	else if (InParameter == StatNamedEventsParam)
	{
		return LOCTEXT("TraceNamedEventsParam", "Capture named events");
	}

	return ProjectLauncher::ICmdLineParametersExtensionFactory::GetCmdLineParameterDisplayName(InParameter);

}


void FInsightsLaunchExtensionInstance::FInsightsCommandLineExtension::CacheTraceChannels()
{
	TraceChannels.Reset();

	FString TraceParamValue = ProjectLauncher::CmdLineUtils::GetParameterValue(GetCommandLine(), TraceParam);
	TraceParamValue.ParseIntoArray(TraceChannels, TEXT(",") );

	TraceChannels.Sort();
}

void FInsightsLaunchExtensionInstance::FInsightsCommandLineExtension::ToggleTraceChannel( const FString& InChannel )
{
	if (TraceChannels.Contains(InChannel))
	{
		TraceChannels.Remove(InChannel);
	}
	else
	{
		TraceChannels.Add(InChannel);
	}

	FString CommandLine = GetCommandLine();
	if (TraceChannels.IsEmpty())
	{
		ProjectLauncher::CmdLineUtils::RemoveParameter(CommandLine, TraceParam);
	}
	else
	{
		FString TraceParamValue = FString::Join( TraceChannels, TEXT(",") );
		ProjectLauncher::CmdLineUtils::UpdateParameterValue( CommandLine, TraceParam, TraceParamValue );
	}
	SetCommandLine(CommandLine);
}

bool FInsightsLaunchExtensionInstance::FInsightsCommandLineExtension::IsTraceChannelEnabled( const FString InChannel ) const
{
	return TraceChannels.Contains(InChannel);
}


void FInsightsLaunchExtensionInstance::FInsightsCommandLineExtension::CustomizeParametersSubmenu( FMenuBuilder& MenuBuilder )
{
#if UE_TRACE_ENABLED || UE_TRACE_MINIMAL_ENABLED
	auto ChannelsMenuDelegate = [this]( FMenuBuilder& MenuBuilder )
	{
		CacheTraceChannels();

		// collect all trace channels
		// note that this will enumerate the channels that are available for the editor / UnrealFrontend, not the ones available to the game
		// UnrealFrontend does not have all channels available
		TArray<FString> AllTraceChannels;
		UE::Trace::EnumerateChannels([](const UE::Trace::FChannelInfo& ChannelInfo, void* User)
		{
			TArray<FString>& AllTraceChannels = *static_cast<TArray<FString>*>(User);
			if (!ChannelInfo.bIsReadOnly)
			{
				FString Channel = FString(ChannelInfo.Name).LeftChop(7); // Remove "Channel" suffix
				AllTraceChannels.Add(Channel);
			}
			return true;
		}, &AllTraceChannels );
		AllTraceChannels.Sort();

		auto SetNone = [this]()
		{
			FString CommandLine = GetCommandLine();
			ProjectLauncher::CmdLineUtils::RemoveParameter(CommandLine, TraceParam);
			SetCommandLine(CommandLine);

			TraceChannels.Reset();
		};

		// add menu item to clear the selected channels
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ClearTraceChannelsLabel", "None"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction( 
				FExecuteAction::CreateLambda(SetNone),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda( [this]() { return (TraceChannels.IsEmpty()); } )
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		MenuBuilder.AddMenuSeparator();

		// add submenu items for each channel
		for ( const FString& Channel : AllTraceChannels )
		{
			MenuBuilder.AddMenuEntry(
				FText::FromString(Channel),
				FText::GetEmpty(),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda( [this, Channel]() { ToggleTraceChannel(Channel); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda( [this, Channel]() { return IsTraceChannelEnabled(Channel); } )
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton);
		}
	};



	MenuBuilder.AddSubMenu(
		LOCTEXT("TraceChannelLabels", "Select Channels"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateLambda(ChannelsMenuDelegate),
		true, // bInOpenSubMenuOnClick
		FSlateIcon(),
		false // bInShouldCloseWindowAfterMenuSelection
	);
#endif
}






TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FInsightsLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
{
	return MakeShared<FInsightsLaunchExtensionInstance>(InArgs);
}

const TCHAR* FInsightsLaunchExtension::GetInternalName() const
{
	return TEXT("Insights");
}

FText FInsightsLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Unreal Insights");
}


#undef LOCTEXT_NAMESPACE
