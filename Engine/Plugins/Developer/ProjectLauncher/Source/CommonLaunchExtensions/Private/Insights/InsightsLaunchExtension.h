// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/LaunchExtension.h"
#include "Extension/CmdLineParametersExtension.h"


class FInsightsLaunchExtensionInstance : public ProjectLauncher::FLaunchExtensionInstance, public ProjectLauncher::ICmdLineParametersExtensionFactory
{
public:
	FInsightsLaunchExtensionInstance( FArgs& InArgs ) : FLaunchExtensionInstance(InArgs) {};
	virtual ~FInsightsLaunchExtensionInstance() = default;

	virtual class ProjectLauncher::ICmdLineParametersExtensionFactory* AsCmdLineParametersFactory() override { return this; }

	// ... ICmdLineParametersExtensionFactory
	virtual TSharedRef<ProjectLauncher::FCmdLineParametersExtension> Create( const ProjectLauncher::FCmdLineParametersExtension::FArgs& InArgs ) override;
	virtual void GetCmdLineParameters( TArray<FString>& OutParameters ) const override;
	virtual FText GetCmdLineParameterDisplayName( const FString& InParameter ) const override;

private:
	class FInsightsCommandLineExtension : public ProjectLauncher::FCmdLineParametersExtension
	{
	public:
		FInsightsCommandLineExtension( const ProjectLauncher::FCmdLineParametersExtension::FArgs& InArgs );


		virtual void CustomizeParametersSubmenu( FMenuBuilder& MenuBuilder ) override;

		void CacheTraceChannels();
		void ToggleTraceChannel( const FString& InChannel );
		bool IsTraceChannelEnabled( const FString InChannel ) const;

		TArray<FString> TraceChannels;
	};

	static const TCHAR* FileParam;
	static const TCHAR* HostParam;
	static const TCHAR* TraceParam;
	static const TCHAR* StatNamedEventsParam;
};


class FInsightsLaunchExtension : public ProjectLauncher::FLaunchExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
	virtual bool IsAlwaysCreated(ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel) const override { return true; }
};