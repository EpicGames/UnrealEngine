// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileWizard/ProfileWizardLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"

#define LOCTEXT_NAMESPACE "FProfileWizardLaunchExtensionInstance"


void FProfileWizardLaunchExtension::MakeCustomExtensionSubmenu( FMenuBuilder& MenuBuilder, ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel )
{
	auto CreateSubMenu = [this, InModel]( FMenuBuilder& MenuBuilder )
	{
		for (const ILauncherProfileWizardPtr& ProfileWizard : InModel->GetProfileManager()->GetProfileWizards())
		{
			auto RunWizard = [this, InModel, ProfileWizard]()
			{
				ProfileWizard->HandleCreateLauncherProfile( InModel->GetProfileManager() );
			};

			MenuBuilder.AddMenuEntry(
				ProfileWizard->GetName(),
				ProfileWizard->GetDescription(),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateLambda(RunWizard) ),
				NAME_None);
		}
	};

	if (InModel->GetProfileManager()->GetProfileWizards().Num() > 0)
	{
		MenuBuilder.AddSubMenu( GetDisplayName(), FText::GetEmpty(), FNewMenuDelegate::CreateLambda(CreateSubMenu));
	}
}



TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FProfileWizardLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
{
	// this extension doesn't need instantation
	return nullptr;
}


const TCHAR* FProfileWizardLaunchExtension::GetInternalName() const
{
	return TEXT("ProfileWizard");
}


FText FProfileWizardLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "New Profile Wizard");
}


void FProfileWizardLaunchExtension::GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const
{
	MenuEntry = FExtensionsMenuEntry::Deprecated;	
}

#undef LOCTEXT_NAMESPACE
