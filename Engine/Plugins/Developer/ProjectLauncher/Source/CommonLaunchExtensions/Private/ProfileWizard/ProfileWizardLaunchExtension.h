// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/LaunchExtension.h"

class FProfileWizardLaunchExtension : public ProjectLauncher::FLaunchExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
	virtual bool IsAlwaysCreated(ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel) const override { return true; }
	virtual void GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const override;
	virtual void MakeCustomExtensionSubmenu(FMenuBuilder& MenuBuilder, ILauncherProfileRef InProfile, TSharedRef<ProjectLauncher::FModel> InModel) override;

};
