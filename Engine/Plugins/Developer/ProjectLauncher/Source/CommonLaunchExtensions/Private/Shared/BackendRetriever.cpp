// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/BackendRetriever.h"
#include "HAL/PlatformMisc.h"
#include "Misc/EngineVersion.h"
#include "Misc/ConfigCacheIni.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "PlatformHttp.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"

FBackendRetriever::FBackendRetriever()
{
	GConfig->GetString(TEXT("ProjectLauncher"), TEXT("BackendQueryUrl"), BackendQueryUrl, GEngineIni);
}

void FBackendRetriever::GetBackendsAsync(FOnBackendRetrievalComplete&& InOnComplete)
{
	if (BackendQueryUrl.IsEmpty())
	{
		return;
	}

	if (bIsGettingBackends)
	{
		if (InOnComplete)
		{
			InOnComplete();
		}
		return;
	}

	bIsGettingBackends = true;
	OnComplete = MoveTemp(InOnComplete);
	ErrorMessage = FString();
	BuildToBackendsMap.Reset();


	// build the backend query request
	FString Stream = FEngineVersion::Current().GetBranch();
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetVerb(TEXT("POST"));
	HttpRequest->SetURL(BackendQueryUrl);
	HttpRequest->SetHeader(TEXT("Content-Type"), TEXT("application/json"));
	HttpRequest->SetContentAsString(FString::Printf(TEXT("{\"stream\":\"%s\"}"), *Stream));
	HttpRequest->OnProcessRequestComplete().BindLambda( 
		[this](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful)
		{
			if (bWasSuccessful && Response->GetResponseCode() == 200)
			{
				ParseBackendFromOutput(Response->GetContentAsString());
			}
			else if (Response)
			{
				ErrorMessage = FString::Printf(TEXT("Failed to query backend. error code %d: %s"), Response->GetResponseCode(), *Response->GetContentAsString());
			}
			else
			{
				ErrorMessage = TEXT("Failed to query backend. no response from server");
			}
			Completed();
		}
	);

	// launch the request
	if (!HttpRequest->ProcessRequest())
	{
		ErrorMessage = FString::Printf(TEXT("Failed to start backend query for stream %s"), *Stream);
		Completed();
	}
}



void FBackendRetriever::ParseBackendFromOutput( const FString& QueryResult)
{
	TSharedPtr<FJsonObject> ResponseJson;
	if (FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(QueryResult), ResponseJson) && ResponseJson.IsValid())
	{
		const TArray<TSharedPtr<FJsonValue>>* AttributesJsonArray;
		if (ResponseJson->TryGetArrayField(TEXT("list"), AttributesJsonArray))
		{
			for (int32 NameIdx = 0; NameIdx < AttributesJsonArray->Num(); ++NameIdx)
			{
				TSharedPtr<FJsonValue> BuildInfoJson = (*AttributesJsonArray)[NameIdx];

				const TSharedPtr<FJsonObject>* BuildInfoObj;
				if (BuildInfoJson->TryGetObject(BuildInfoObj))
				{
					FString EpicApp, BuildInfo;
					if ((*BuildInfoObj)->TryGetStringField(TEXT("epic_app"), EpicApp) &&
						(*BuildInfoObj)->TryGetStringField(TEXT("build_info"), BuildInfo))
					{
						// Build is in the format of "++<Project>+<StreamName>-CL-<Changelist>-<Postfix>"
						FString ClKey = TEXT("CL-");
						int32 ClIndex = BuildInfo.Find(ClKey);
						if (ClIndex != INDEX_NONE)
						{
							// Remove the postfix part after changelist number
							FString ChangelistStr = BuildInfo.Mid(ClIndex + ClKey.Len());
							int32 DashIndex = ChangelistStr.Find(TEXT("-"));
							if (DashIndex != INDEX_NONE)
							{
								ChangelistStr = ChangelistStr.Left(DashIndex);
							}
							// Return the first backend name matching the changelist, there may be more than one matching backends.
							int32 Changelist = FCString::Atoi(*ChangelistStr);
							BuildToBackendsMap.FindOrAdd(Changelist).Add(EpicApp.ToLower());
						}
					}
				}
			}
		}
	}

	// Sort backends into alphabetical order
	for (auto& Pair : BuildToBackendsMap)
	{
		TArray<FString>& Backends = Pair.Value;
		Backends.Sort();
	}
}



void FBackendRetriever::Completed()
{
	bIsGettingBackends = false;
	if (OnComplete)
	{
		OnComplete();
		OnComplete = nullptr;
	}
}



bool FBackendRetriever::IsGettingBackends() const
{
	return bIsGettingBackends;
}



bool FBackendRetriever::IsConfigured() const
{
	return !BackendQueryUrl.IsEmpty();
}

const TMap<int32, TArray<FString>>& FBackendRetriever::GetBuildToBackendsMap() const
{
	return BuildToBackendsMap;
}



const FString& FBackendRetriever::GetErrorMessage() const
{
	return ErrorMessage;
}
