// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/UGSBuildInfoRetriever.h"
#include "Misc/EngineVersion.h"
#include "Misc/ConfigCacheIni.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "PlatformHttp.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"
#include "Interfaces/IHttpResponse.h"

FUGSBuildInfoRetriever::FUGSBuildInfoRetriever()
{
	GConfig->GetString(TEXT("ProjectLauncher"), TEXT("HordeMetadataUrl"), HordeMetadataUrl, GEngineIni);
}

void FUGSBuildInfoRetriever::GetUGSBuildInfoAsync( const FString& InProjectName, const TArrayView<int32>& InChangelists, FOnUGSBuildInfoRetrievalComplete&& InOnComplete )
{
	if (HordeMetadataUrl.IsEmpty())
	{
		return;
	}

	if (bIsGettingUGSBuildInfo)
	{
		if (InOnComplete)
		{
			InOnComplete();
		}
		return;
	}

	bIsGettingUGSBuildInfo = true;
	OnComplete = MoveTemp(InOnComplete);
	ErrorMessage = FString();
	BuildToUGSBuildInfoMap.Reset();

	// must have a project
	if (InProjectName.IsEmpty())
	{
		ErrorMessage = FString::Printf(TEXT("Failed to query badges. No project specified"));
		Completed();
		return;
	}


	// build the UGS badge query URL, limiting to a reasonable maximum length
	const int MaxUrlLength = 2000;
	FString Stream = FEngineVersion::Current().GetBranch();
	FString URL = HordeMetadataUrl;

	URL += TEXT("?stream=") + FPlatformHttp::UrlEncode(Stream.ToLower());
	URL += TEXT("&project=") + FPlatformHttp::UrlEncode(InProjectName.ToLower());
	for (int32 Changelist : InChangelists)
	{
		URL += FString::Printf( TEXT("&change=%d"), Changelist);

		if (URL.Len() > MaxUrlLength)
		{
			break;
		}
	}

	// build the UGS query request
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->SetVerb(TEXT("GET"));
	HttpRequest->SetURL(URL);
	HttpRequest->OnProcessRequestComplete().BindLambda(
		[this, ProjectName = InProjectName](FHttpRequestPtr Request, FHttpResponsePtr Response, bool bWasSuccessful)
		{
			if (bWasSuccessful && Response->GetResponseCode() == 200)
			{
				ParseUGSBuildInfoFromQuery(Response->GetContentAsString(), ProjectName);
			}
			else if (Response)
			{
				ErrorMessage = FString::Printf(TEXT("Failed to query badges. error code %d: %s"), Response->GetResponseCode(), *Response->GetContentAsString());
			}
			else
			{
				ErrorMessage = TEXT("Failed to query badges. no response from server");
			}
			Completed();
		}
	);

	// launch the request
	if (!HttpRequest->ProcessRequest())
	{
		ErrorMessage = FString::Printf(TEXT("Failed to start backend query for stream %s"), *Stream);
		Completed();
	}
}



void FUGSBuildInfoRetriever::ParseUGSBuildInfoFromQuery(const FString& Text, const FString& ProjectName )
{

	TSharedPtr<FJsonObject> ResponseJson;
	if (FJsonSerializer::Deserialize(TJsonReaderFactory<>::Create(Text), ResponseJson) && ResponseJson.IsValid())
	{
		const TArray<TSharedPtr<FJsonValue>>* JsonItems;
		if (ResponseJson->TryGetArrayField(TEXT("items"), JsonItems))
		{
			for (const TSharedPtr<FJsonValue>& JsonItem : (*JsonItems))
			{
				int32 Changelist;
				FString Project;
				const TSharedPtr<FJsonObject>* JsonItemObj;
				const TArray<TSharedPtr<FJsonValue>>* JsonBadges;

				if ( (JsonItem->TryGetObject(JsonItemObj) &&
						(*JsonItemObj)->TryGetNumberField(TEXT("change"), Changelist) && 
						(*JsonItemObj)->TryGetStringField(TEXT("project"), Project) &&
						(*JsonItemObj)->TryGetArrayField(TEXT("badges"), JsonBadges)))
				{
					if (!Project.Equals(ProjectName, ESearchCase::IgnoreCase))
					{
						continue;
					}

					TSharedPtr<FUGSBuildInfo> BuildInfo;
					if (BuildToUGSBuildInfoMap.Contains(Changelist))
					{
						BuildInfo = BuildToUGSBuildInfoMap[Changelist];
					}
					else
					{
						BuildInfo = BuildToUGSBuildInfoMap.Add(Changelist, MakeShared<FUGSBuildInfo>());
					}


					for (const TSharedPtr<FJsonValue>& JsonBadge : (*JsonBadges))
					{
						const TSharedPtr<FJsonObject>* JsonBadgeObj;
						FString BadgeName;
						FString BadgeURL;
						FString BadgeState;
						if (JsonBadge->TryGetObject(JsonBadgeObj) &&
							(*JsonBadgeObj)->TryGetStringField(TEXT("name"), BadgeName) &&
							(*JsonBadgeObj)->TryGetStringField(TEXT("url"), BadgeURL) &&
							(*JsonBadgeObj)->TryGetStringField(TEXT("state"), BadgeState))
						{
							FUGSBadge& Badge = BuildInfo->Badges.AddDefaulted_GetRef();
							Badge.Name = BadgeName;
							Badge.URL = BadgeURL;
							Badge.State = ParseUGSBadgeState(BadgeState);
						}
					}

					const TArray<TSharedPtr<FJsonValue>>* JsonUsers;
					if ((*JsonItemObj)->TryGetArrayField(TEXT("users"), JsonUsers))
					{
						BuildInfo->NumUsers += JsonUsers->Num();
						for (const TSharedPtr<FJsonValue>& JsonUser : (*JsonUsers))
						{
							FString UserVote;
							const TSharedPtr<FJsonObject>* JsonUserObj;
							if (JsonUser->TryGetObject(JsonUserObj) &&
								(*JsonUserObj)->TryGetStringField(TEXT("vote"), UserVote))
							{
								if (UserVote.Equals(TEXT("CompileSuccess"), ESearchCase::IgnoreCase))
								{
									BuildInfo->NumSuccess++;
								}
								else if (UserVote.Equals(TEXT("CompileFailure"), ESearchCase::IgnoreCase))
								{
									BuildInfo->NumFailed++;
								}
								else
								{
									ErrorMessage += FString::Printf(TEXT("Unknown UGS badge vote type: %s\n"), *UserVote);
								}
							}
						}
					}
				}
			}
		}
	}
}



FUGSBuildInfoRetriever::FUGSBadge::EState FUGSBuildInfoRetriever::ParseUGSBadgeState( const FString& State )
{
	static const struct
	{
		FUGSBadge::EState State;
		const TCHAR* Text;
	}
	States[] = 
	{
		{ FUGSBadge::EState::Starting, TEXT("Starting") },
		{ FUGSBadge::EState::Failure,  TEXT("Failure")  },
		{ FUGSBadge::EState::Warning,  TEXT("Warning")  },
		{ FUGSBadge::EState::Success,  TEXT("Success")  },
		{ FUGSBadge::EState::Skipped,  TEXT("Skipped")  },
	};

	for (int32 Index = 0; Index < UE_ARRAY_COUNT(States); Index++)
	{
		if (State.Equals(States[Index].Text, ESearchCase::IgnoreCase))
		{
			return States[Index].State;
		}
	}

	return FUGSBadge::EState::Unknown;
}



void FUGSBuildInfoRetriever::Completed()
{
	bIsGettingUGSBuildInfo = false;
	if (OnComplete)
	{
		OnComplete();
		OnComplete = nullptr;
	}
}



bool FUGSBuildInfoRetriever::IsGettingUGSBuildInfo() const
{
	return bIsGettingUGSBuildInfo;
}



bool FUGSBuildInfoRetriever::IsConfigured() const
{
	return !HordeMetadataUrl.IsEmpty();
}

const TMap<int32, TSharedRef<FUGSBuildInfoRetriever::FUGSBuildInfo>>& FUGSBuildInfoRetriever::GetBuildToUGSBuildInfoMap()
{
	return BuildToUGSBuildInfoMap;
}



const FString& FUGSBuildInfoRetriever::GetErrorMessage() const
{
	return ErrorMessage;
}
