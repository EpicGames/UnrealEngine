// Copyright Epic Games, Inc. All Rights Reserved.

#include "UgsSync/UgsSyncLaunchExtension.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Serialization/JsonTypes.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/FileHelper.h"
#include "Misc/EngineVersion.h"
#include "Dom/JsonObject.h"
#include "Dom/JsonValue.h"
#include "Templates/SharedPointer.h"
#include "PlatformHttp.h"
#include "HttpModule.h"
#include "Interfaces/IHttpRequest.h"


#define LOCTEXT_NAMESPACE "FUGSSyncLaunchExtensionInstance"

FUGSSyncLaunchExtensionInstance::FUGSSyncLaunchExtensionInstance(FArgs& InArgs)
	: ProjectLauncher::FCustomUATCommandLaunchExtensionInstance(InArgs)
{
}

FUGSSyncLaunchExtensionInstance::~FUGSSyncLaunchExtensionInstance()
{
}

void FUGSSyncLaunchExtensionInstance::InternalInitialize()
{
	FLaunchExtensionInstance::InternalInitialize();

	RefreshLatestSyncedState();
}

const FString FUGSSyncLaunchExtensionInstance::GetUATCommandInternalName() const
{
	return TEXT("UgsSyncExtension.UGSSync");
}

void FUGSSyncLaunchExtensionInstance::CustomizeTree(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData)
{
	FLaunchExtensionInstance::CustomizeTree(ProfileTreeData);

	auto CurrentlySynced = [this]()
	{
		FText Value;
		FNumberFormattingOptions Options;
		Options.SetUseGrouping(false);
		Value = FText::AsNumber(SyncedChangelist, &Options);
		return Value;
	};

	auto IsEnabled = []()
	{
		return !GIsEditor;
	};

	AddDefaultHeading(ProfileTreeData)
		.AddWidget(LOCTEXT("SyncedChangelist", "Synced changelist"),
			{
				.IsVisible = [this] { return true; },
				.IsEnabled = IsEnabled,
			},
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(9, 1)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text_Lambda(CurrentlySynced)
			]
		)
		.AddWidget(LOCTEXT("SyncTo", "Sync to"),
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(1, 1)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.OnTextCommitted(this, &FUGSSyncLaunchExtensionInstance::OnSelectedChangelistCommitted)
				.HintText_Lambda(CurrentlySynced)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4, 1)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text_Lambda([this]() { return FText::Format(LOCTEXT("ChangelistDelta", "\u2206 = {0}"), {SelectedChangelist != 0 ? SyncedChangelist - SelectedChangelist : 0 }); })
			]
		)
		.AddBoolean(LOCTEXT("DryRun", "Dry run"),
			{
				.GetValue = [this]()			{ return IsDryRunEnabled(); },
				.SetValue = [this](bool Val)	{ SetDryRunEnabled(Val); }
			}
		);
}

void FUGSSyncLaunchExtensionInstance::CustomizeUATCommandLine(FString& InOutCommandLine)
{
	FString CommandLine;
	CommandLine += TEXT(" -SyncUGS");

	CommandLine += FString::Printf(TEXT(" -project=\"%s\" "), *GetProfile()->GetProjectPath());
	CommandLine += FString::Printf(TEXT(" -CL=\"%d\" "), SelectedChangelist != 0 ? SelectedChangelist : SyncedChangelist);

	FString Branch = FEngineVersion::Current().GetBranch();
	Branch = Branch.Replace(TEXT("//"), TEXT("")).Replace(TEXT("/"), TEXT("-")).ToLower();
	CommandLine += FString::Printf(TEXT(" -Stream=%s"), *Branch);

	if (bDryRun)
	{
		CommandLine += TEXT(" -DryRun");
	}

	InOutCommandLine += CommandLine;
}

void FUGSSyncLaunchExtensionInstance::OnUATCommandAdded(ILauncherProfileUATCommandRef InUATCommand)
{
	InUATCommand->SetUATCommand(TEXT("BuildAcquire")); // RunUAT BuildAcquire [...] 
}

void FUGSSyncLaunchExtensionInstance::OnSelectedChangelistCommitted(const FText& InText, ETextCommit::Type CommitType)
{
	if (!InText.IsNumeric())
	{
		SelectedChangelist = 0;
	}
	else
	{
		SelectedChangelist = FCString::Atoi(*InText.ToString());
	}
}

void FUGSSyncLaunchExtensionInstance::RefreshLatestSyncedState()
{
	FString StateJsonPath = FPaths::Combine(FPaths::RootDir(), ".ugs/state.json");

	FString StateJsonContents;
	if (!FFileHelper::LoadFileToString(StateJsonContents, *StateJsonPath))
	{
		return;
	}

	const TSharedRef< TJsonReader<> >& Reader = TJsonReaderFactory<>::Create(StateJsonContents);
	TSharedPtr<FJsonObject> StateObject;
	if (!FJsonSerializer::Deserialize(Reader, StateObject) && !StateObject.IsValid())
	{
		return;
	}

	int32 NewlySyncedChangelist = SyncedChangelist;
	if (StateObject->TryGetNumberField(TEXTVIEW("LastSyncChangeNumber"), NewlySyncedChangelist))
	{
		if (NewlySyncedChangelist >= 0 && NewlySyncedChangelist != SyncedChangelist)
		{
			SyncedChangelist = NewlySyncedChangelist;
			GetProfile()->GetCustomStringProperties().Add(TEXT("SyncedChangelist"), ::LexToString(SyncedChangelist));
		}
	}
}

FUGSSyncLaunchExtension::FUGSSyncLaunchExtension()
{
}

TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FUGSSyncLaunchExtension::CreateInstanceForProfile(ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs)
{
	return MakeShared<FUGSSyncLaunchExtensionInstance>(InArgs);
}

const TCHAR* FUGSSyncLaunchExtension::GetInternalName() const
{
	return TEXT("UGSSync");
}

FText FUGSSyncLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "UGS Sync");
}

#undef LOCTEXT_NAMESPACE
