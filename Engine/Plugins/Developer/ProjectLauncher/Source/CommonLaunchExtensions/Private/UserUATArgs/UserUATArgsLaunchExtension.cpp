// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserUATArgs/UserUATArgsLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"

#define LOCTEXT_NAMESPACE "FUserUATArgsLaunchExtensionInstance"



FUserUATArgsLaunchExtensionInstance::FUserUATArgsLaunchExtensionInstance( FArgs& InArgs )
	: ProjectLauncher::FBuildCookRunCommandExtensionInstance(InArgs)
	, Owner(StaticCastSharedRef<FUserUATArgsLaunchExtension>(InArgs.Extension))
{
}

TSharedRef<ProjectLauncher::FBuildCookRunExtension> FUserUATArgsLaunchExtensionInstance::CreateBuildCookRunExtension( const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs )
{
	return MakeShared<FBuildCookRunInstance>(InArgs);
}




FUserUATArgsLaunchExtensionInstance::FBuildCookRunInstance::FBuildCookRunInstance( const FArgs& InArgs )
	: ProjectLauncher::FBuildCookRunExtension( InArgs )
{
}


void FUserUATArgsLaunchExtensionInstance::FBuildCookRunInstance::CustomizeTree( ProjectLauncher::FLaunchProfileTreeNode& ProfileTreeNode )
{
	AddDefaultHeading(ProfileTreeNode)
		.AddCommandLineString( LOCTEXT("ExtraParamsLabel", "Additional UAT Parameters"),
			{
				.GetValue = [this]()              { return GetConfigString(EConfig::PerProfile, TEXT("ExtraUATParams")); },
				.SetValue = [this](FString Value) { SetConfigString(EConfig::PerProfile, TEXT("ExtraUATParams"), Value); },
				.GetDefaultValue = [this]()       { return FString(); },
			},
			ProjectLauncher::CmdLineType::UAT
		);
	;
}

void FUserUATArgsLaunchExtensionInstance::FBuildCookRunInstance::CustomizeUATCommandLine(FString& InOutCommandLine)
{
	FString ExtraUATParams = GetConfigString(EConfig::PerProfile, TEXT("ExtraUATParams")).TrimStartAndEnd();
	if (!ExtraUATParams.IsEmpty())
	{
		InOutCommandLine += TEXT(" ");
		InOutCommandLine += ExtraUATParams;
	}
}








TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FUserUATArgsLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
{
	return MakeShared<FUserUATArgsLaunchExtensionInstance>(InArgs);
}

const TCHAR* FUserUATArgsLaunchExtension::GetInternalName() const
{
	return TEXT("UserUATArgs");
}

FText FUserUATArgsLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Custom UAT Parameters");
}



#undef LOCTEXT_NAMESPACE
