// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/BuildCookRunCommandExtension.h"


class FUserUATArgsLaunchExtensionInstance : public ProjectLauncher::FBuildCookRunCommandExtensionInstance
{
public:
	FUserUATArgsLaunchExtensionInstance( FArgs& InArgs );
	virtual ~FUserUATArgsLaunchExtensionInstance() = default;

	virtual TSharedRef<ProjectLauncher::FBuildCookRunExtension> CreateBuildCookRunExtension( const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs ) override;

protected:
	TSharedRef<class FUserUATArgsLaunchExtension> Owner;

	class FBuildCookRunInstance : public ProjectLauncher::FBuildCookRunExtension
	{
	public:
		FBuildCookRunInstance(const ProjectLauncher::FBuildCookRunExtension::FArgs& InArgs );
		virtual void CustomizeTree( ProjectLauncher::FLaunchProfileTreeNode& ProfileTreeNode ) override;
		virtual void CustomizeUATCommandLine(FString& InOutCommandLine) override;

	};
};


class FUserUATArgsLaunchExtension : public ProjectLauncher::FBuildCookRunCommandExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
};