// Copyright Epic Games, Inc. All Rights Reserved.

#include "UserUATCommand/UserUATCommandLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"


#define LOCTEXT_NAMESPACE "FUserUATCommandLaunchExtensionInstance"

FUserUATCommandLaunchExtensionInstance::FUserUATCommandLaunchExtensionInstance(FArgs& InArgs)
	: ProjectLauncher::FCustomUATCommandLaunchExtensionInstance(InArgs)
{
}


void FUserUATCommandLaunchExtensionInstance::CustomizeTree(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData)
{
	FLaunchExtensionInstance::CustomizeTree(ProfileTreeData);

	AddDefaultHeading(ProfileTreeData)
		.AddString(LOCTEXT("UATCommandLabel", "UAT Command"),
			{
				.GetValue = [this]()              { return GetUATCommand()->GetUATCommand(); },
				.SetValue = [this](FString Value) { GetUATCommand()->SetUATCommand(*Value); GetProfile()->RefreshCustomWarningsAndErrors(GetUATCommand()); },
				.Validation = ProjectLauncher::FValidation({TEXT("Validation_UserUATCmd_NoCommand")}),
			}
		)
		.AddCommandLineString(LOCTEXT("ExtraParamsLabel", "UAT Parameters"),
			{
				.GetValue = [this]()              { return GetConfigString(EConfig::PerProfile, TEXT("UATParams")); },
				.SetValue = [this](FString Value) { SetConfigString(EConfig::PerProfile, TEXT("UATParams"), Value); },
				.GetDefaultValue = [this]()       { return FString(); },
			},
			ProjectLauncher::CmdLineType::UAT
		);
	;
}

void FUserUATCommandLaunchExtensionInstance::CustomizeUATCommandLine(FString& InOutCommandLine)
{
	FString ExtraUATParams = GetConfigString(EConfig::PerProfile, TEXT("UATParams")).TrimStartAndEnd();
	if (!ExtraUATParams.IsEmpty())
	{
		InOutCommandLine += TEXT(" ");
		InOutCommandLine += ExtraUATParams;
	}
}

void FUserUATCommandLaunchExtensionInstance::OnValidateProfile()
{
	FString UATCommandName = GetUATCommand()->GetUATCommand().TrimStartAndEnd();
	if (UATCommandName.IsEmpty() || UATCommandName.Contains(TEXT(" ")) || UATCommandName.Contains("-") )
	{
		GetProfile()->AddCustomError(TEXT("Validation_UserUATCmd_NoCommand"), LOCTEXT("Validation_UserUATCmd_NoCommand", "You must specify a valid UAT command"), GetUATCommand());
	}
}

void FUserUATCommandLaunchExtensionInstance::OnUATCommandAdded(ILauncherProfileUATCommandRef InUATCommand)
{
	InUATCommand->SetUATCommand(TEXT(""));
}




TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FUserUATCommandLaunchExtension::CreateInstanceForProfile(ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs)
{
	return MakeShared<FUserUATCommandLaunchExtensionInstance>(InArgs);
}

const TCHAR* FUserUATCommandLaunchExtension::GetInternalName() const
{
	return TEXT("UserUATCommand");
}

FText FUserUATCommandLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Custom UAT Command");
}

#undef LOCTEXT_NAMESPACE
