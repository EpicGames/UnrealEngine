// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/CustomUATCommandLaunchExtension.h"

class FUserUATCommandLaunchExtensionInstance : public ProjectLauncher::FCustomUATCommandLaunchExtensionInstance
{
public:
	FUserUATCommandLaunchExtensionInstance(FArgs& InArgs);

	virtual void CustomizeTree(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData) override;
	virtual void CustomizeUATCommandLine(FString& InOutCommandLine) override;
	
protected:
	virtual void OnUATCommandAdded(ILauncherProfileUATCommandRef InUATCommand) override;
	virtual void OnValidateProfile() override;
};

class FUserUATCommandLaunchExtension: public ProjectLauncher::FCustomUATCommandLaunchExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile(ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
};