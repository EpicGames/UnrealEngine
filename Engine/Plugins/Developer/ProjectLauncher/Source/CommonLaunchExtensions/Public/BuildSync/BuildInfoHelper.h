// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Experimental/BuildListRetriever.h"
#include "Shared/UGSBuildInfoRetriever.h"
#include "Shared/BackendRetriever.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include <atomic>

#define UE_API COMMONLAUNCHEXTENSIONS_API


class FBuildInfoHelper : public TSharedFromThis<FBuildInfoHelper>
{
public:
	UE_API FBuildInfoHelper();
	UE_API ~FBuildInfoHelper();

	struct FBuildInfo : public TSharedFromThis<FBuildInfo, ESPMode::ThreadSafe>
	{
		TSharedPtr<UE::Zen::Build::FBuildListRetriever::FBuildGroup> Group;

		TSharedPtr<FUGSBuildInfoRetriever::FUGSBuildInfo> UGSBuildInfo;
		TArray<FString> Backends;

		TSet<FName> Platforms;
		TMap<FName,TArray<FString>> PlatformToArtifacts;
	};


	UE_API void SetProjectName( const FString& InProjectName );
	UE_API void SetBuildType( const FString& InBuildType );
	UE_API FString GetBuildType() const;
	UE_API void SetBuildsRefreshedHandler(TFunction<void()> OnBuildsRefreshed);


	// build service connection
	UE_API void Connect();
	UE_API bool IsConnected() const;
	UE_API bool IsConfigured() const;
	UE_API FText GetErrorText() const;

	// build list refresh
	UE_API void Refresh( bool bFullRefresh );
	UE_API bool IsRefreshing() const;

	// list filtering
	struct FFilter
	{
		FTimespan MaxAge;
		int32 MaxItems = 50;
	};
	void SetFilter( const FFilter& Filter );



	UE_API TSharedRef<UE::Zen::Build::FBuildServiceInstance> GetServiceInstance() const;

	UE_API const TArray<TSharedPtr<FBuildInfo, ESPMode::ThreadSafe>>& GetBuildInfos() const;

	UE_API const TMap<FString,TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildType, ESPMode::ThreadSafe>>& GetBuildTypesMap() const;

	UE_API const TSet<FString>& GetAllKnownNamedArtifacts() const;

	UE_API FString GetPlatformNameFromNamedArtifact( const FString& NamedArtifact, const UE::Zen::Build::FBuildServiceInstance::FBuildRecord* BuildRecord = nullptr ) const;

	UE_API static const TCHAR* DefaultBuildType;

	

private:


	TFunction<void()> OnBuildsRefreshed;
	FString ProjectName;
	FString BuildType;
	std::atomic<bool> bBuildListRetrieverInitComplete;
	TSharedPtr<UE::Zen::Build::FBuildListRetriever> BuildListRetriever;
	TSharedPtr<FBackendRetriever> BackendRetriever;
	TSharedPtr<FUGSBuildInfoRetriever> UGSBuildInfoRetriever;
	TSet<FString> KnownPlatformNames;


	FFilter Filter;




	// build list details
	void RefreshBackend();
	void RefreshUGSBuildInfo();
	void RefreshBuildGroups();
	void CreateFilteredBuildGroups( const TArrayView<TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildGroup>>& BuildGroups ); 

	TArray<TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildGroup>> FilteredBuildGroups;
	std::atomic<bool> bBuildGroupsRequestComplete;

	// build info
	std::atomic<bool> bIsRefreshing = false;
	void CheckRefreshComplete();
	void RebuildBuildInfos();
	TArray<TSharedPtr<FBuildInfo, ESPMode::ThreadSafe>> BuildInfos;
	TMap<FString,TSharedRef<UE::Zen::Build::FBuildListRetriever::FBuildType, ESPMode::ThreadSafe>> BuildTypesMap;
	TSet<FString> AllKnownNamedArtifacts;



};

#undef UE_API
