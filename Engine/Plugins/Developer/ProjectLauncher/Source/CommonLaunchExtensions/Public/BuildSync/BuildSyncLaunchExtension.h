// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/CustomUATCommandLaunchExtension.h"
#include "Experimental/BuildServerInterface.h"
#include "BuildSync/BuildInfoHelper.h"

#include <atomic>

#define UE_API COMMONLAUNCHEXTENSIONS_API

namespace PlatformInfo
{
	struct FTargetPlatformInfo;
}

class FBuildSyncLaunchExtensionInstance : public ProjectLauncher::FCustomUATCommandLaunchExtensionInstance
{
public:
	FBuildSyncLaunchExtensionInstance( FArgs& InArgs );
	virtual ~FBuildSyncLaunchExtensionInstance();

	virtual void InternalInitialize() override;

	virtual void CustomizeTree( ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData ) override;
	virtual void CustomizeUATCommandLine( FString& InOutCommandLine ) override;

	virtual void OnUATCommandAdded( ILauncherProfileUATCommandRef UATCommand ) override;
	void OnOtherUATCommandAdded(const ILauncherProfileUATCommandRef& UATCommand);


	virtual void OnValidateProfile() override;
	virtual void OnProjectChanged() override;
	virtual void OnPropertyChanged() override;

	UE_API TArray<FString> GetSelectedBuildBackends() const;

private:
	UE::Zen::Build::FBuildServiceInstance::EConnectionState GetBuildServiceConnectionState() const;
	void OnBuildsRefreshed();
	void RefreshFilteredBuilds();

	void SetBuildType( FString InBuildType );
	FString GetBuildType() const;
	void SetNamedArtifacts( TArray<FString> InNamedArtifacts );
	TArray<FString> GetNamedArtifacts() const;
	void SetDestinationDirectory(FString InPath);
	FString GetDestinationDirectory() const;

	void AutoSelectBestBuild();
	void AutoSelectNamedArtifacts();

	TSharedRef<SWidget> CreateBuildStorageConnectionWidget();
	TSharedRef<SWidget> CreateBuildFilterWidget();
	TSharedRef<SWidget> CreateBuildTypeSelectionWidget();
	TSharedRef<SWidget> CreateNamedArtifactSelectionWidget();

	void SetSelectedBuild(TSharedPtr<FBuildInfoHelper::FBuildInfo> Build);
	TSharedPtr<FBuildInfoHelper::FBuildInfo> GetSelectedBuild() const;
	TArray<FString> GetAvailableArtifactsForSelectedBuild() const;

	bool GetBuildSuitability( TSharedPtr<FBuildInfoHelper::FBuildInfo> Build, FText* OutReason ) const;
	void CacheUnsuitableBuilds();
	
	TSharedPtr<FBuildInfoHelper::FBuildInfo> SelectedBuild;

	TSharedPtr<FBuildInfoHelper> BuildInfo;
	TSharedRef<class FBuildSyncLaunchExtension> Owner;
	TSharedPtr<class SBuildSyncBuildCombo> BuildCombo;
	TArray<FString> CachedNamedArtifacts;
	TMap<TSharedPtr<FBuildInfoHelper::FBuildInfo>, FText> CachedUnsuitableBuilds;
	TArray<TSharedPtr<FBuildInfoHelper::FBuildInfo>> FilteredBuildInfos;

	bool bShowPreflights = false;
	bool bShowUnsuitable = false;
	FString FilterText;
	FString DestinationDirectory;

	TSet<const PlatformInfo::FTargetPlatformInfo*> GetCookPlatforms() const;
	TSet<const PlatformInfo::FTargetPlatformInfo*> CachedCookPlatforms;
};


class FBuildSyncLaunchExtension : public ProjectLauncher::FCustomUATCommandLaunchExtension
{
public:
	FBuildSyncLaunchExtension();

	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
};

#undef UE_API