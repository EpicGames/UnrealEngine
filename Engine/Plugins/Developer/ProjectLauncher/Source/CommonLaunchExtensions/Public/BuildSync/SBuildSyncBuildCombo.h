// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Fonts/SlateFontInfo.h"
#include "BuildSync/BuildInfoHelper.h"

#define UE_API COMMONLAUNCHEXTENSIONS_API

template<typename ItemType> class SComboBox;


class SBuildSyncBuildCombo
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TSharedPtr<FBuildInfoHelper::FBuildInfo> );
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FGetItemSuitability, TSharedPtr<FBuildInfoHelper::FBuildInfo>, FText* );

	SLATE_BEGIN_ARGS(SBuildSyncBuildCombo)
		: _OptionsSource()
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		{}
		SLATE_ARGUMENT(const TArray<TSharedPtr<FBuildInfoHelper::FBuildInfo>>*, OptionsSource)
		SLATE_ARGUMENT(TSharedPtr<SWidget>, FilterWidget)
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_ATTRIBUTE(TSharedPtr<FBuildInfoHelper::FBuildInfo>, SelectedItem);
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_ATTRIBUTE(TSet<FName>, RequiredPlatforms)
		SLATE_EVENT(FGetItemSuitability, GetItemSuitability)
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )
	SLATE_END_ARGS()

public:
	UE_API void Construct(const FArguments& InArgs);
	UE_API void RefreshOptions();

protected:
	const TArray<TSharedPtr<FBuildInfoHelper::FBuildInfo>>* OptionsSource;
	TAttribute<TSharedPtr<FBuildInfoHelper::FBuildInfo>> SelectedItem;
	TAttribute<TSet<FName>> RequiredPlatforms;
	FOnSelectionChanged OnSelectionChanged;
	FGetItemSuitability GetItemSuitability;

	TSharedRef<ITableRow> OnGenerateBuildInfoRow( TSharedPtr<FBuildInfoHelper::FBuildInfo> BuildInfo, const TSharedRef<STableViewBase>& OwnerTable) const;
	void OnBuildInfoSelectionChanged( TSharedPtr<FBuildInfoHelper::FBuildInfo> BuildInfo, ESelectInfo::Type InSelectInfo );
	FText GetSelectedBuildInfoName() const;

	TSharedPtr<SComboButton> ComboButton;
	TSharedPtr<SListView<TSharedPtr<FBuildInfoHelper::FBuildInfo>>> BuildInfosListView;

	FText GetAge( const FDateTime& DateTime ) const;
};

#undef UE_API
