// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Containers/Map.h"
#include <atomic>


class FBackendRetriever : public TSharedFromThis<FBackendRetriever>
{
public:
	typedef TUniqueFunction<void()> FOnBackendRetrievalComplete;

	FBackendRetriever();

	void GetBackendsAsync(FOnBackendRetrievalComplete&& InOnComplete);
	bool IsGettingBackends() const;
	bool IsConfigured() const;

	const TMap<int32, TArray<FString>>& GetBuildToBackendsMap() const;
	const FString& GetErrorMessage() const;

private:

	void ParseBackendFromOutput(const FString& QueryResult);
	void Completed();

	FOnBackendRetrievalComplete OnComplete;
	std::atomic<bool> bIsGettingBackends = false;

	TMap<int32, TArray<FString>> BuildToBackendsMap;
	FString ErrorMessage;
	FString BackendQueryUrl;
};
