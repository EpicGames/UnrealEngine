// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include <atomic>


class FUGSBuildInfoRetriever : public TSharedFromThis<FUGSBuildInfoRetriever>
{
public:
	struct FUGSBadge
	{
		enum class EState : uint8
		{
			Starting,
			Failure,
			Warning,
			Success,
			Skipped,

			Unknown,
			MAX
		};

		FString Name;
		FString URL;
		EState State;
	};

	struct FUGSBuildInfo : public TSharedFromThis<FUGSBuildInfo, ESPMode::ThreadSafe>
	{
		int32 NumUsers = 0;
		int32 NumSuccess = 0;
		int32 NumFailed = 0;

		TArray<FUGSBadge> Badges;
	};

	typedef TUniqueFunction<void()> FOnUGSBuildInfoRetrievalComplete;

	FUGSBuildInfoRetriever();

	void GetUGSBuildInfoAsync( const FString& InProjectName, const TArrayView<int32>& InChangelists, FOnUGSBuildInfoRetrievalComplete&& InOnComplete );
	bool IsGettingUGSBuildInfo() const;
	bool IsConfigured() const;

	const TMap<int32, TSharedRef<FUGSBuildInfo>>& GetBuildToUGSBuildInfoMap();
	const FString& GetErrorMessage() const;

private:

	void ParseUGSBuildInfoFromQuery(const FString& Text, const FString& ProjectName);
	FUGSBadge::EState ParseUGSBadgeState(const FString& Text);
	void Completed();

	FOnUGSBuildInfoRetrievalComplete OnComplete;
	std::atomic<bool> bIsGettingUGSBuildInfo = false;

	TMap<int32, TSharedRef<FUGSBuildInfo>> BuildToUGSBuildInfoMap;
	FString ErrorMessage;
	FString HordeMetadataUrl;
};
