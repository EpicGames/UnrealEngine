// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/CustomUATCommandLaunchExtension.h"

class FUGSSyncLaunchExtensionInstance : public ProjectLauncher::FCustomUATCommandLaunchExtensionInstance
{
public:
	FUGSSyncLaunchExtensionInstance(FArgs& InArgs);
	virtual ~FUGSSyncLaunchExtensionInstance();

	virtual void InternalInitialize() override;

	virtual void CustomizeTree(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData) override;
	virtual void CustomizeUATCommandLine(FString& InOutCommandLine) override;
	
	const FString GetUATCommandInternalName() const;

protected:
	virtual void OnUATCommandAdded(ILauncherProfileUATCommandRef InUATCommand) override;

private:
	void OnSelectedChangelistCommitted(const FText& InText, ETextCommit::Type CommitType);
	void RefreshLatestSyncedState();

	bool IsDryRunEnabled() const { return bDryRun; }
	void SetDryRunEnabled(bool Val) { bDryRun = Val; }

private:
	int32 SelectedChangelist = 0;
	int32 SyncedChangelist = 0;
	bool bDryRun = false;
};

class FUGSSyncLaunchExtension: public ProjectLauncher::FCustomUATCommandLaunchExtension
{
public:
	FUGSSyncLaunchExtension();
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile(ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
};