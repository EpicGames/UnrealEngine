// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extension/AutomatedTestLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"
#include "Styling/ProjectLauncherStyle.h"

#define LOCTEXT_NAMESPACE "FAutomatedTestLaunchExtensionInstance"


namespace ProjectLauncher
{
	FAutomatedTestLaunchExtensionInstance::FAutomatedTestLaunchExtensionInstance( FArgs& InArgs )
		: FUATCommandLaunchExtensionInstanceBase(InArgs)
	{
		check(UATCommand->AsAutomatedTest().IsValid());
	}

	void FAutomatedTestLaunchExtensionInstance::OnAdded()
	{
		ILauncherProfileAutomatedTestPtr AutomatedTest = GetTest();

		// set the defaults, placing automated tests at the end by default
		FModel::FUATCommandsDetail UATCommandsDetail; 
		GetModel()->GetUATCommandsDetail(GetProfile(), UATCommandsDetail);
		AutomatedTest->SetOrder( UATCommandsDetail.MaxOrder + 1 );
		AutomatedTest->SetDescription( GetExtension()->GetDisplayName().ToString() );

		OnTestAdded(AutomatedTest.ToSharedRef());

		BroadcastPropertyChanged();
	}

	void FAutomatedTestLaunchExtensionInstance::OnRemoved()
	{
		ILauncherProfileAutomatedTestPtr ExistingAutomatedTest = GetTest();
		if (ExistingAutomatedTest.IsValid())
		{
			OnTestRemoved(ExistingAutomatedTest.ToSharedRef());
			GetProfile()->RemoveAutomatedTest(ExistingAutomatedTest->GetInternalName());
		}
		BroadcastPropertyChanged();
	}

	ILauncherProfileAutomatedTestPtr FAutomatedTestLaunchExtensionInstance::GetTest() const
	{
		return UATCommand->AsAutomatedTest();
	}





	void FAutomatedTestLaunchExtension::GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const
	{
		MenuEntry = FExtensionsMenuEntry::AutomatedTests;
	}

	FSlateIcon FAutomatedTestLaunchExtension::GetIcon() const
	{
		return FSlateIcon( FProjectLauncherStyle::GetStyleSetName(), "Icons.Task.TestAutomation");
	}

	ILauncherProfileUATCommandRef FAutomatedTestLaunchExtension::CreateUATCommand(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel) const
	{
		return InProfile->FindOrAddAutomatedTest( nullptr, GetInternalName() );
	}


}

#undef LOCTEXT_NAMESPACE
