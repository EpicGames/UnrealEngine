// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extension/BuildCookRunCommandExtension.h"
#include "Model/ProjectLauncherModel.h"
#include "Styling/ProjectLauncherStyle.h"

#define LOCTEXT_NAMESPACE "FBuildCookRunCommandExtensionInstance"


namespace ProjectLauncher
{
	FBuildCookRunExtension::FBuildCookRunExtension( const FArgs& InArgs )
		: Profile(InArgs.Profile)
		, BuildCookRun(InArgs.BuildCookRun)
		, DefaultBuildCookRun(InArgs.DefaultBuildCookRun)
		, Model(InArgs.Model)
		, Extension(InArgs.Extension)
		, ExtensionInstance(InArgs.ExtensionInstance)
		, OwnerTreeData(InArgs.OwnerTreeData)
	{
		Profile->OnCustomUATCommandValidation().AddRaw( this, &FBuildCookRunExtension::OnUATCommandValidation );
		Profile->OnProjectChanged().AddRaw( this, &FBuildCookRunExtension::OnProjectChanged);
	}

	FBuildCookRunExtension::~FBuildCookRunExtension()
	{
		Profile->OnCustomUATCommandValidation().RemoveAll(this);
		Profile->OnProjectChanged().RemoveAll(this);
	}

	void FBuildCookRunExtension::OnUATCommandValidation( ILauncherProfileUATCommandRef UATCommand )
	{
		if (UATCommand == BuildCookRun && UATCommand->IsEnabled())
		{
			OnValidate();
		}
	}

	FLaunchProfileTreeNode& FBuildCookRunExtension::AddDefaultHeading(FLaunchProfileTreeNode& ProfileTreeData) const
	{
		return ProfileTreeData.AddSubHeading(Extension->GetInternalName(), Extension->GetDisplayName(), INT32_MAX );
	}

	FString FBuildCookRunExtension::GetConfigString(EConfig Config, const TCHAR* Name, const TCHAR* DefaultValue) const
	{
		return ExtensionInstance->GetConfigString(Config, *GetConfigKeyName(Name), DefaultValue);
	}

	bool FBuildCookRunExtension::GetConfigBool(EConfig Config, const TCHAR* Name, bool DefaultValue) const
	{
		return ExtensionInstance->GetConfigBool(Config, *GetConfigKeyName(Name), DefaultValue);
	}

	int32 FBuildCookRunExtension::GetConfigInteger(EConfig Config, const TCHAR* Name, int32 DefaultValue) const
	{
		return ExtensionInstance->GetConfigInteger(Config, *GetConfigKeyName(Name), DefaultValue);
	}

	float FBuildCookRunExtension::GetConfigFloat(EConfig Config, const TCHAR* Name, float DefaultValue) const
	{
		return ExtensionInstance->GetConfigFloat(Config, *GetConfigKeyName(Name), DefaultValue);
	}

	void FBuildCookRunExtension::SetConfigString(EConfig Config, const TCHAR* Name, const FString& Value) const
	{
		ExtensionInstance->SetConfigString(Config, *GetConfigKeyName(Name), Value);
	}

	void FBuildCookRunExtension::SetConfigBool(EConfig Config, const TCHAR* Name, const bool& Value) const
	{
		ExtensionInstance->SetConfigBool(Config, *GetConfigKeyName(Name), Value);
	}

	void FBuildCookRunExtension::SetConfigInteger(EConfig Config, const TCHAR* Name, const int32& Value) const
	{
		ExtensionInstance->SetConfigInteger(Config, *GetConfigKeyName(Name), Value);
	}

	void FBuildCookRunExtension::SetConfigFloat(EConfig Config, const TCHAR* Name, const float& Value) const
	{
		ExtensionInstance->SetConfigFloat(Config, *GetConfigKeyName(Name), Value);
	}

	FString FBuildCookRunExtension::GetConfigKeyName(const TCHAR* Name) const
	{
		return FString::Printf(TEXT("%s.%s"), BuildCookRun->GetInternalName(), Name);
	}












	FBuildCookRunCommandExtensionInstance::FBuildCookRunCommandExtensionInstance( FArgs& InArgs )
		: FLaunchExtensionInstance(InArgs)
	{
		GetProfile()->OnUATCommandAdded().AddRaw( this, &FBuildCookRunCommandExtensionInstance::OnUATCommandAdded);
		GetProfile()->OnUATCommandRemoved().AddRaw( this, &FBuildCookRunCommandExtensionInstance::OnUATCommandRemoved);

	}

	FBuildCookRunCommandExtensionInstance::~FBuildCookRunCommandExtensionInstance()
	{
		BuildCookRunExtensions.Reset();
		GetProfile()->OnUATCommandAdded().RemoveAll(this );
		GetProfile()->OnUATCommandRemoved().RemoveAll(this );
	}

	void FBuildCookRunCommandExtensionInstance::InternalInitialize()
	{
		FLaunchExtensionInstance::InternalInitialize();

		// create all build cook run extensions if we support them		
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : GetProfile()->GetBuildCookRunCommands())
		{
			bool bEnabledByDefault = IsBuildCookRunExtensionEnabledByDefault(BuildCookRun);
			if (GetConfigBool(EConfig::PerProfile, *GetConfigKeyName(TEXT("Enabled"), BuildCookRun), bEnabledByDefault))
			{
				CreateBuildCookRunExtensionInternal(BuildCookRun);
			}
		}
	}

	void FBuildCookRunCommandExtensionInstance::CustomizeTree( FLaunchProfileTreeData& ProfileTreeData )
	{
		FLaunchExtensionInstance::CustomizeTree(ProfileTreeData);

		for (const TTuple<ILauncherProfileUATCommandPtr,TSharedRef<FBuildCookRunExtension>>& Pair : BuildCookRunExtensions)
		{
			if (GetModel()->CanUseSimplifiedLayout(GetProfile()))
			{
				Pair.Value->CustomizeTree(ProfileTreeData.GetHeading(FLaunchProfileTreeData::GeneralSettingsSectionName));
			}
			else
			{
				const bool bIsPrimaryHeading = true;
				FLaunchProfileTreeNode& ProfileTreeNode = ProfileTreeData.FindOrAddHeading(Pair.Key.ToSharedRef(), bIsPrimaryHeading);

				Pair.Value->CustomizeTree(ProfileTreeNode);
			}
		}
	}

	FString FBuildCookRunCommandExtensionInstance::GetConfigKeyName( const TCHAR* Name, const ILauncherProfileBuildCookRunRef& BuildCookRun) const
	{
		return FString::Printf(TEXT("%s.%s"), BuildCookRun->GetInternalName(), Name);
	}

	bool FBuildCookRunCommandExtensionInstance::HasActiveBuildCookRunExtension() const
	{
		return (BuildCookRunExtensions.Num() > 0);
	}

	TSharedRef<FBuildCookRunExtension> FBuildCookRunCommandExtensionInstance::CreateBuildCookRunExtension( const FBuildCookRunExtension::FArgs& InArgs )
	{
		checkNoEntry();
		TSharedPtr<FBuildCookRunExtension> Invalid;
		return Invalid.ToSharedRef();

	}
	
	TSharedRef<FBuildCookRunExtension> FBuildCookRunCommandExtensionInstance::CreateBuildCookRunExtensionInternal( const ILauncherProfileBuildCookRunRef& InBuildCookRun )
	{
		TSharedRef<FBuildCookRunExtension>* ExistingInstancePtr = BuildCookRunExtensions.Find(InBuildCookRun.ToSharedPtr());
		if (ExistingInstancePtr != nullptr)
		{
			return (*ExistingInstancePtr);
		}

		FBuildCookRunExtension::FArgs Args
		{
			.Profile = GetProfile(),
			.BuildCookRun = InBuildCookRun,
			.DefaultBuildCookRun = GetModel()->GetCustomDefaultBuildCookRun(),
			.Model = GetModel(),
			.Extension = GetExtension(),
			.ExtensionInstance = AsShared(),
			.OwnerTreeData = GetOwnerTreeData(),
		};

		TSharedRef<FBuildCookRunExtension> NewInstance = CreateBuildCookRunExtension(Args);

		BuildCookRunExtensions.Add(InBuildCookRun.ToSharedPtr(), NewInstance);

		return NewInstance;
	}

	void FBuildCookRunCommandExtensionInstance::CustomizeBuildCookRunExtensionMenu( FMenuBuilder& InMenuBuilder, const ILauncherProfileBuildCookRunRef& InBuildCookRun )
	{
		auto IsEnabled = [this, InBuildCookRun]()
		{
			return BuildCookRunExtensions.Contains(InBuildCookRun.ToSharedPtr());
		};

		auto ToggleEnabled = [this, InBuildCookRun, IsEnabled]()
		{
			bool bEnabled = !IsEnabled();
			SetConfigBool(EConfig::PerProfile, *GetConfigKeyName(TEXT("Enabled"), InBuildCookRun), bEnabled);
			RequestFullTreeRebuild();
		};

		auto CanToggle = [this, InBuildCookRun, IsEnabled]
		{
			bool bWantToEnable = !IsEnabled();
			return CanToggleBuildCookRunExtension(InBuildCookRun, bWantToEnable);
		};

		auto GetToolTipText = [InBuildCookRun, CanToggle, IsEnabled]
		{
			bool bIsEnabled = IsEnabled();
			bool bCanToggle = CanToggle();

			if (bIsEnabled && !bCanToggle)
			{
				return LOCTEXT("CannotRemoveExtensionTooltip", "The extension cannot be removed at this time");
			}
			else if (!bIsEnabled && !bCanToggle)
			{
				return LOCTEXT("CannotAddExtensionTooltip", "The extension cannot be added at this time");
			}

			return FText::GetEmpty();
		};


		InMenuBuilder.AddMenuEntry(
			GetExtension()->GetDisplayName(),
			GetToolTipText(),
			GetExtension()->GetIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(ToggleEnabled),
				FCanExecuteAction::CreateLambda(CanToggle),
				FIsActionChecked::CreateLambda(IsEnabled)
			),
			NAME_None,
			EUserInterfaceActionType::Check);
	}

	bool FBuildCookRunCommandExtensionInstance::IsBuildCookRunExtensionEnabledByDefault( const ILauncherProfileBuildCookRunRef& InBuildCookRun ) const
	{
		return false;
	}

	bool FBuildCookRunCommandExtensionInstance::CanToggleBuildCookRunExtension( const ILauncherProfileBuildCookRunRef& InBuildCookRun, bool bWantToEnable ) const
	{
		return true;
	}



	void FBuildCookRunCommandExtensionInstance::OnUATCommandAdded( const ILauncherProfileUATCommandRef& UATCommand )
	{
		ILauncherProfileBuildCookRunPtr BuildCookRun = UATCommand->AsBuildCookRun();
		if (BuildCookRun.IsValid())
		{
			if (!BuildCookRunExtensions.Contains(UATCommand.ToSharedPtr()) && IsBuildCookRunExtensionEnabledByDefault(BuildCookRun.ToSharedRef()))
			{
				CreateBuildCookRunExtensionInternal(BuildCookRun.ToSharedRef());
				RequestFullTreeRebuild();
			}
		}
	}

	void FBuildCookRunCommandExtensionInstance::OnUATCommandRemoved(const ILauncherProfileUATCommandRef& UATCommand)
	{
		ILauncherProfileBuildCookRunPtr BuildCookRun = UATCommand->AsBuildCookRun();
		if (BuildCookRun.IsValid())
		{
			if (BuildCookRunExtensions.Contains(UATCommand.ToSharedPtr()))
			{
				BuildCookRunExtensions.Remove(UATCommand.ToSharedPtr());
				RequestFullTreeRebuild();
			}
		}
	}

	bool FBuildCookRunCommandExtensionInstance::CustomizeUATCommandLine( const ILauncherProfileUATCommandRef& InUATCommand, FString& InOutCommandLine )
	{
		bool bResult = FLaunchExtensionInstance::CustomizeUATCommandLine(InUATCommand, InOutCommandLine);

		TSharedRef<FBuildCookRunExtension>* ExistingInstancePtr = BuildCookRunExtensions.Find(InUATCommand.ToSharedPtr());
		if (ExistingInstancePtr != nullptr)
		{
			(*ExistingInstancePtr)->CustomizeUATCommandLine(InOutCommandLine);
			return true;
		}

		return bResult;
	}

	void FBuildCookRunCommandExtensionInstance::OnPropertyChanged() 
	{
		for (const TTuple<ILauncherProfileUATCommandPtr,TSharedRef<FBuildCookRunExtension>>& Pair : BuildCookRunExtensions)
		{
			Pair.Value->OnPropertyChanged();
		}
	}
}

#undef LOCTEXT_NAMESPACE
