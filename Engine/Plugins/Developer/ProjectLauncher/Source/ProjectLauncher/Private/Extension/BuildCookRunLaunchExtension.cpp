// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extension/BuildCookRunLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"
#include "Styling/ProjectLauncherStyle.h"

#define LOCTEXT_NAMESPACE "FBuildCookRunLaunchExtensionInstance"


namespace ProjectLauncher
{
	FBuildCookRunLaunchExtensionInstance::FBuildCookRunLaunchExtensionInstance( FArgs& InArgs )
		: FUATCommandLaunchExtensionInstanceBase(InArgs)
	{
		check(UATCommand->AsBuildCookRun().IsValid());
	}


	
	void FBuildCookRunLaunchExtension::GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const
	{
		MenuEntry = FExtensionsMenuEntry::Default;
	}

	FSlateIcon FBuildCookRunLaunchExtension::GetIcon() const
	{
		return FSlateIcon(FProjectLauncherStyle::GetStyleSetName(), "Icons.Task.Launch");
	}

	ILauncherProfileUATCommandRef FBuildCookRunLaunchExtension::CreateUATCommand(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel) const
	{
		ILauncherProfileBuildCookRunRef BuildCookRun = InProfile->FindOrAddBuildCookRunCommand( nullptr, GetInternalName() );

		InModel->SetDefaults(InProfile, BuildCookRun);

		return BuildCookRun;

	}

	const TCHAR* FBuildCookRunLaunchExtension::GetInternalName() const
	{
		return TEXT("BuildCookRun");
	}

	FText FBuildCookRunLaunchExtension::GetDisplayName() const
	{
		return LOCTEXT("BuildCookRunExtensionName", "Build, Cook & Run");
	}

	TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FBuildCookRunLaunchExtension::CreateInstanceForProfile( FLaunchExtensionInstance::FArgs& InArgs )
	{
		return MakeShared<FBuildCookRunLaunchExtensionInstance>(InArgs);
	}

	bool FBuildCookRunLaunchExtension::CanBeCreated( ILauncherProfileRef InProfile, TSharedRef<FModel> InModel ) const
	{
		// cannot add a new BuildCookRun command to the basic launch profile (no technical reason - just want to avoid making it too complex)
		return !InModel->IsBasicLaunchProfile(InProfile);
	}
}

#undef LOCTEXT_NAMESPACE
