// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extension/CmdLineParametersExtension.h"
#include "Utils/LauncherCmdLineUtils.h"
#include "Model/ProjectLauncherModel.h"

#define LOCTEXT_NAMESPACE "FCmdLineParametersExtensionInstance"


namespace ProjectLauncher
{
	FCmdLineParametersExtension::FCmdLineParametersExtension( const FArgs& InArgs )
		: Profile(InArgs.ExtensionInstance->GetProfile())
		, Model(InArgs.Model)
		, Extension(InArgs.ExtensionInstance->GetExtension())
		, ExtensionInstance(InArgs.ExtensionInstance)
		, SetCommandLineDelegate(InArgs.SetCommandLine)
		, GetCommandLineDelegate(InArgs.GetCommandLine)
		, TypeName(InArgs.TypeName)
	{
		check(SetCommandLineDelegate.IsSet());
		check(GetCommandLineDelegate.IsSet());

		Profile->OnProjectChanged().AddRaw( this, &FCmdLineParametersExtension::OnProjectChanged);
	}

	FCmdLineParametersExtension::~FCmdLineParametersExtension()
	{
		Profile->OnProjectChanged().RemoveAll(this);
	}

	void FCmdLineParametersExtension::SetCommandLine( const FString& InCommandLine )
	{
		SetCommandLineDelegate(InCommandLine);
		ExtensionInstance->BroadcastPropertyChanged();
	}

	FString FCmdLineParametersExtension::GetCommandLine() const
	{
		return GetCommandLineDelegate();
	}


	static TMap<uint64,TArray<TSharedRef<FCmdLineParametersExtension>>> GCmdLineParametersExtensions;

	void ICmdLineParametersExtensionFactory::MakeCmdLineParametersSubmenu(FMenuBuilder& MenuBuilder, uint64 InstanceID, const FString& TypeName, TSharedRef<FModel> Model, TSharedRef<FLaunchExtensionInstance> ExtensionInstance, const FLaunchProfileTreeNode::FSetString& SetValue, const FLaunchProfileTreeNode::FGetString& GetValue )
	{
		ProjectLauncher::ICmdLineParametersExtensionFactory* CmdLineParametersFactory = ExtensionInstance->AsCmdLineParametersFactory();
		check(CmdLineParametersFactory); // should have been verified before calling

		// get the command line instances for this field
		if (!GCmdLineParametersExtensions.Contains(InstanceID))
		{
			GCmdLineParametersExtensions.Add(InstanceID);
		}
		TArray<TSharedRef<FCmdLineParametersExtension>>& CmdLineExtensions = GCmdLineParametersExtensions[InstanceID];

		// find or create the extension instance for this field
		TSharedRef<FCmdLineParametersExtension>* CmdLineExtensionPtr = CmdLineExtensions.FindByPredicate( [ExtensionInstance]( const TSharedRef<FCmdLineParametersExtension>& CmdLineParametersInstance )
		{
			return CmdLineParametersInstance->GetExtensionInstance() == ExtensionInstance;
		});
		TSharedPtr<FCmdLineParametersExtension> CmdLineExtension;
		if (CmdLineExtensionPtr)
		{
			CmdLineExtension = (*CmdLineExtensionPtr);
		}
		else
		{
			// create & cache a new extension instance
			FCmdLineParametersExtension::FArgs Args
			{
				.Model = Model,
				.ExtensionInstance = ExtensionInstance,
				.SetCommandLine = SetValue,
				.GetCommandLine = GetValue,
				.TypeName = TypeName,
			};

			CmdLineExtension = CmdLineParametersFactory->Create(Args);
			CmdLineExtensions.Add(CmdLineExtension.ToSharedRef());
		}

		// create the menu
		TArray<FString> Parameters;
		CmdLineParametersFactory->GetCmdLineParameters(Parameters);
		if (Parameters.Num() > 0)
		{
			auto ToggleParameter = [CmdLineExtension]( FString Parameter )
			{
				FString CommandLine = CmdLineExtension->GetCommandLine();
				if (CmdLineUtils::IsParameterUsed(CommandLine, Parameter))
				{
					CmdLineUtils::RemoveParameter(CommandLine, Parameter);
				}
				else
				{
					CmdLineUtils::AddParameter(CommandLine, Parameter);
				}
				CmdLineExtension->SetCommandLine(CommandLine);
			};


			for (const FString& Parameter : Parameters)
			{
				FText ParameterDisplayName = CmdLineParametersFactory->GetCmdLineParameterDisplayName(Parameter);
				FText ParameterToolTip = FText::FromString(Parameter);

				MenuBuilder.AddMenuEntry( 
					ParameterDisplayName,
					ParameterToolTip,
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda(ToggleParameter, Parameter),
						FCanExecuteAction(),
						FIsActionChecked::CreateLambda( [CmdLineExtension, Parameter]() { return CmdLineUtils::IsParameterUsed(CmdLineExtension->GetCommandLine(), Parameter); } )
					),
					NAME_None,
					EUserInterfaceActionType::Check);
			}
		}

		CmdLineExtension->CustomizeParametersSubmenu(MenuBuilder);
	}

	TSharedRef<FCmdLineParametersExtension> ICmdLineParametersExtensionFactory::Create( const FCmdLineParametersExtension::FArgs& InArgs )
	{
		return MakeShared<ProjectLauncher::FCmdLineParametersExtension>(InArgs);
	}

	FText ICmdLineParametersExtensionFactory::GetCmdLineParameterDisplayName( const FString& InParameter ) const
	{
		return FText::FromString(InParameter);
	}

	void ICmdLineParametersExtensionFactory::DestroyCmdLineParametersExtensions(TSharedPtr<FLaunchExtensionInstance> Instance)
	{
		// remove all launch command line instances that come from the given extension, recording any instances that become empty
		TSet<uint64> EmptyInstanceIDs;
		for (TTuple<uint64,TArray<TSharedRef<FCmdLineParametersExtension>>>& Pair : GCmdLineParametersExtensions)
		{
			Pair.Value.RemoveAll( [Instance]( const TSharedRef<FCmdLineParametersExtension>& CmdLineParametersExtension )
			{
				return CmdLineParametersExtension->GetExtensionInstance() == Instance;
			});

			if (Pair.Value.Num() == 0)
			{
				EmptyInstanceIDs.Add(Pair.Key);
			}
		}

		// remove all empty instances
		for (uint64 EmptyInstanceID : EmptyInstanceIDs)
		{
			GCmdLineParametersExtensions.Remove(EmptyInstanceID);
		}
	}
}

#undef LOCTEXT_NAMESPACE
