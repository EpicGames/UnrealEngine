// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extension/CustomUATCommandLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"

#define LOCTEXT_NAMESPACE "FCustomUATCommandLaunchExtensionInstance"


namespace ProjectLauncher
{
	FCustomUATCommandLaunchExtensionInstance::FCustomUATCommandLaunchExtensionInstance( FArgs& InArgs )
		: FUATCommandLaunchExtensionInstanceBase(InArgs)
	{
	}

	void FCustomUATCommandLaunchExtensionInstance::OnAdded()
	{
		// set the defaults, placing uat commands at the start by default
		FModel::FUATCommandsDetail UATCommandsDetail; 
		GetModel()->GetUATCommandsDetail(GetProfile(), UATCommandsDetail);
		UATCommand->SetOrder( UATCommandsDetail.MinOrder - 1 ); 
		UATCommand->SetDescription( GetExtension()->GetDisplayName().ToString() );

		OnUATCommandAdded(UATCommand);

		BroadcastPropertyChanged();
	}
	
	void FCustomUATCommandLaunchExtensionInstance::OnRemoved()
	{
		OnUATCommandRemoved(UATCommand);
		GetProfile()->RemoveUATCommand(UATCommand->GetInternalName());

		BroadcastPropertyChanged();
	}




	void FCustomUATCommandLaunchExtension::GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const
	{
		MenuEntry = FExtensionsMenuEntry::UATCommands;
	}
	
	ILauncherProfileUATCommandRef FCustomUATCommandLaunchExtension::CreateUATCommand(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel) const
	{
		return InProfile->FindOrAddUATCommand( nullptr, GetInternalName() );
	}
}

#undef LOCTEXT_NAMESPACE
