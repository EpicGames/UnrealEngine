// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extension/LaunchExtension.h"
#include "Extension/CmdLineParametersExtension.h"
#include "Styling/ProjectLauncherStyle.h"
#include "Model/ProjectLauncherModel.h"
#include "Utils/LauncherCmdLineUtils.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "ILauncherServicesModule.h"
#include "PlatformInfo.h"
#include <atomic>

#define LOCTEXT_NAMESPACE "FLaunchExtensionInstance"


namespace ProjectLauncher
{
	static TArray<TSharedPtr<FLaunchExtension>> GExtensions;
	static TMap<ILauncherProfilePtr, TArray<TSharedPtr<FLaunchExtensionInstance>>> GExtensionInstances;
	static TMap<TSharedPtr<FLaunchExtension>,TArray<TSharedPtr<FLaunchExtensionInstance>>> GInstancesPerExtension;
	static FCriticalSection GExtensionsCS;
	static std::atomic<bool> GExtensionsDestroyed = false;

	TArray<TSharedPtr<FLaunchExtensionInstance>> GetProfileExtensionInstances(ILauncherProfilePtr Profile)
	{
		// take a snapshot of the extension instances for this profile
		TArray<TSharedPtr<FLaunchExtensionInstance>> ExtensionInstances;
		{
			FScopeLock Lock(&GExtensionsCS);
			if (GExtensionInstances.Contains(Profile))
			{
				ExtensionInstances = GExtensionInstances[Profile];
			}
		}
		return ExtensionInstances;
	}

	void ShutdownLaunchExtensions()
	{
		if (!GExtensionsDestroyed)
		{
			GExtensions.Reset();
			GExtensionInstances.Reset();
			GInstancesPerExtension.Reset();

			GExtensionsDestroyed = true;
		}
	}


	void RegisterExtension( TSharedRef<FLaunchExtension> Extension )
	{
		check(!GExtensionsDestroyed);

		FScopeLock Lock(&GExtensionsCS);
		GExtensions.Add(Extension);
		GInstancesPerExtension.Add(Extension);
	}

	void UnregisterExtension( TSharedRef<FLaunchExtension> Extension )
	{
		if (!GExtensionsDestroyed)
		{
			FScopeLock Lock(&GExtensionsCS);
			GExtensions.Remove(Extension);
			GInstancesPerExtension.Remove(Extension);

			for (TTuple<ILauncherProfilePtr, TArray<TSharedPtr<FLaunchExtensionInstance>>>& Pair : GExtensionInstances)
			{
				TArray<TSharedPtr<FLaunchExtensionInstance>>& ExtensionInstances = Pair.Value;
				ExtensionInstances.RemoveAll( [Extension](TSharedPtr<FLaunchExtensionInstance> ExtensionInstance)
				{
					return ExtensionInstance->GetExtension() == Extension;
				});
			}
		}
	}


	void ApplyExtensionVariables( const ILauncherProfileRef& InProfile, FString& InOutCommandLine, TSharedRef<FModel> InModel, EBuildTargetType InBuildTargetType )
	{
		TArray<TSharedPtr<FLaunchExtensionInstance>> ExtensionInstances = GetProfileExtensionInstances(InProfile);

		for (TSharedPtr<FLaunchExtensionInstance> ExtensionInstance : ExtensionInstances)
		{
			if (ExtensionInstance.IsValid())
			{
				// apply variable substitutions
				TArray<FString> Variables;
				if (ExtensionInstance->GetExtensionVariables(Variables))
				{
					for (const FString& Variable : Variables)
					{
						if (InOutCommandLine.Contains(Variable, ESearchCase::IgnoreCase))
						{
							FString VariableValue;
							if (ExtensionInstance->GetExtensionVariableValue( Variable, VariableValue ))
							{
								InOutCommandLine.ReplaceInline( *Variable, *VariableValue, ESearchCase::IgnoreCase );
							}
						}
					}
				}

				// allow for advanced command line customization
				if (InBuildTargetType == EBuildTargetType::Unknown)
				{
					ExtensionInstance->CustomizeLaunchCommandLine( InOutCommandLine );
				}
				else
				{
					ExtensionInstance->CustomizeTargetLaunchCommandLine( InOutCommandLine, InBuildTargetType );
				}
			}
		}
	}


	void ApplyExtensionVariablesForUATCommand( FUATCommandPostProcessParameters& InPostProcessParameters, TSharedRef<FModel> InModel )
	{
		TArray<TSharedPtr<FLaunchExtensionInstance>> ExtensionInstances = GetProfileExtensionInstances(InPostProcessParameters.Profile);

		for (TSharedPtr<FLaunchExtensionInstance> ExtensionInstance : ExtensionInstances)
		{
			if (ExtensionInstance.IsValid())
			{
				// apply variable substitutions
				TArray<FString> Variables;
				if (ExtensionInstance->GetExtensionVariables(Variables))
				{
					for (const FString& Variable : Variables)
					{
						if (InPostProcessParameters.UATCommandLine.Contains(Variable, ESearchCase::IgnoreCase))
						{
							FString VariableValue;
							if (ExtensionInstance->GetExtensionVariableValue( Variable, VariableValue ))
							{
								InPostProcessParameters.UATCommandLine.ReplaceInline( *Variable, *VariableValue, ESearchCase::IgnoreCase );
							}
						}
					}
				}

				// allow for advanced command line customization
				bool bKnownCommand = ExtensionInstance->CustomizeUATCommandLine( InPostProcessParameters.UATCommand, InPostProcessParameters.UATCommandLine );

				// don't skip this command if something knows about it
				InPostProcessParameters.bShouldBeSkipped &= !bKnownCommand;
			}
		}
	}



	void RemoveExtensionInstancesForProfile( const ILauncherProfileRef& Profile )
	{
		FScopeLock Lock(&GExtensionsCS);

		if (GExtensionInstances.Contains(Profile))
		{
			TArray<TSharedPtr<FLaunchExtensionInstance>>& ExtensionInstances = GExtensionInstances.FindChecked(Profile.ToSharedPtr());
			for (TSharedPtr<FLaunchExtensionInstance>& ExtensionInstance : ExtensionInstances)
			{
				if (ExtensionInstance->GetProfile() == Profile)
				{
					ICmdLineParametersExtensionFactory::DestroyCmdLineParametersExtensions(ExtensionInstance);
				}
			}

			GExtensionInstances.Remove(Profile.ToSharedPtr());
		}

		for (TTuple<TSharedPtr<FLaunchExtension>,TArray<TSharedPtr<FLaunchExtensionInstance>>>& Pair : GInstancesPerExtension)
		{
			TArray<TSharedPtr<FLaunchExtensionInstance>>& ExtensionInstances = Pair.Value;
			ExtensionInstances.RemoveAll( [Profile](TSharedPtr<FLaunchExtensionInstance> ExtensionInstance)
			{
				return ExtensionInstance->GetProfile() == Profile;
			});
		}
	}



	TArray<TSharedRef<FLaunchExtension>> FLaunchExtension::GetExtensions()
	{
		FScopeLock Lock(&GExtensionsCS);

		TArray<TSharedRef<FLaunchExtension>> Result;
		for ( const TSharedPtr<FLaunchExtension>& Extension : GExtensions)
		{
			Result.Add(Extension.ToSharedRef());
		}

		return Result;
	}


	TArray<TSharedPtr<FLaunchExtensionInstance>> FLaunchExtension::CreateExtensionInstancesForProfile( ILauncherProfileRef InProfile, TSharedRef<FModel> InModel, FLaunchProfileTreeData& InOwnerTreeData )
	{
		check(!GExtensionsDestroyed);

		// make sure the extensions for this profile haven't been created already
		{
			FScopeLock Lock(&GExtensionsCS);
			const TArray<TSharedPtr<FLaunchExtensionInstance>>* ExistingInstances = GExtensionInstances.Find(InProfile.ToSharedPtr());
			if (ExistingInstances != nullptr)
			{
				return (*ExistingInstances);
			}
		}

		// take a snapshot of the extension list
		TArray<TSharedPtr<FLaunchExtension>> Extensions;
		{
			FScopeLock Lock(&GExtensionsCS);
			Extensions = GExtensions;
		}

		// cache all UAT commands in the profile, organised by type
		TMap<FString,TArray<ILauncherProfileUATCommandRef>> UATCommandsByInternalName;
		for (const ILauncherProfileUATCommandRef& UATCommand : InProfile->GetUATCommands() )
		{
			if (!UATCommandsByInternalName.Contains(UATCommand->GetUserTypeName()))
			{
				UATCommandsByInternalName.Add(UATCommand->GetUserTypeName());
			}
			UATCommandsByInternalName[UATCommand->GetUserTypeName()].Add(UATCommand);
		}


		// attempt to instantiate all extensions
		for (TSharedPtr<FLaunchExtension> Extension : Extensions)
		{
			if (Extension->IsUATCommandManager())
			{
				const TArray<ILauncherProfileUATCommandRef>* UATCommandsPtr = UATCommandsByInternalName.Find(Extension->GetInternalName());
				if (UATCommandsPtr)
				{
					for (const ILauncherProfileUATCommandRef& UATCommand : (*UATCommandsPtr))
					{
						CreateExtensionInstance(Extension.ToSharedRef(), InProfile, InModel, InOwnerTreeData, UATCommand.ToSharedPtr());
					}
				}

			}
			else
			{
				const bool* ValuePtr = InProfile->GetCustomBoolProperties().Find( FString::Printf(TEXT("%s.Instantiated"), Extension->GetInternalName() ) );
				bool bEnabled = ValuePtr && (*ValuePtr);

				if (Extension->IsAlwaysCreated(InProfile, InModel) || Extension->IsCreatedByDefault(InProfile, InModel) || bEnabled)
				{
					CreateExtensionInstance(Extension.ToSharedRef(), InProfile, InModel, InOwnerTreeData);
				}
			}
		}

		// return all extensions
		{
			FScopeLock Lock(&GExtensionsCS);
			const TArray<TSharedPtr<FLaunchExtensionInstance>>* ExistingInstances = GExtensionInstances.Find(InProfile.ToSharedPtr());
			if (ExistingInstances != nullptr)
			{
				return (*ExistingInstances);
			}
		}

		return TArray<TSharedPtr<FLaunchExtensionInstance>>();
	}

	TSharedPtr<FLaunchExtensionInstance> FLaunchExtension::CreateExtensionInstance( TSharedRef<FLaunchExtension> InExtension, ILauncherProfileRef InProfile, TSharedRef<FModel> InModel, FLaunchProfileTreeData& InOwnerTreeData, ILauncherProfileUATCommandPtr InUATCommand )
	{
		check(!GExtensionsDestroyed);

		TSharedPtr<FLaunchExtensionInstance> ExistingInstance = GetExtensionInstance(InExtension, InProfile);
		if (ExistingInstance.IsValid())
		{
			return ExistingInstance;
		}

		if (!InExtension->CanBeCreated(InProfile, InModel))
		{
			return nullptr;
		}

		// look up the default profile
		ILauncherProfileRef DefaultProfile = InModel->IsBasicLaunchProfile(InProfile) ? InModel->GetDefaultBasicLaunchProfile() : InModel->GetDefaultCustomLaunchProfile();

		

		FLaunchExtensionInstance::FArgs Args
		{
			.Profile = InProfile,
			.DefaultProfile = DefaultProfile,
			.Model = InModel,
			.Extension = InExtension,
			.OwnerTreeData = InOwnerTreeData,
			.UATCommand = InUATCommand,
		};
		TSharedPtr<FLaunchExtensionInstance> ExtensionInstance = InExtension->CreateInstanceForProfile(Args);

		if (ExtensionInstance.IsValid())
		{
			ExtensionInstance->InternalInitialize();

			// if this extension needs to be created dynamically, remember we have created it for next time and intialize it
			if (!InExtension->IsAlwaysCreated(InProfile, InModel) && !InExtension->IsUATCommandManager())
			{
				InProfile->GetCustomBoolProperties().Add( FString::Printf(TEXT("%s.Instantiated"), InExtension->GetInternalName() ), true );
				ExtensionInstance->OnAdded();
			}

			// add it to the global caches
			{
				FScopeLock Lock(&GExtensionsCS);
				if (!GExtensionInstances.Contains(InProfile.ToSharedPtr()))
				{
					GExtensionInstances.Add(InProfile.ToSharedPtr());
				}
				GExtensionInstances.FindChecked(InProfile.ToSharedPtr()).Add(ExtensionInstance);
				GInstancesPerExtension.FindChecked(InExtension.ToSharedPtr()).Add(ExtensionInstance);
			}
		}

		return ExtensionInstance;
	}

	void FLaunchExtension::DestroyExtensionInstance( TSharedRef<FLaunchExtensionInstance> InExtensionInstance, ILauncherProfileRef InProfile, TSharedRef<FModel> InModel)
	{
		check(!GExtensionsDestroyed);

		TSharedRef<FLaunchExtension> Extension = InExtensionInstance->GetExtension();

		// can't destroy auto-instantiated extensions
		check(!Extension->IsAlwaysCreated(InProfile, InModel));

		if (!Extension->IsUATCommandManager())
		{
			InProfile->GetCustomBoolProperties().Add( FString::Printf(TEXT("%s.Instantiated"), Extension->GetInternalName() ), false );
		}

		// remove it from the global caches
		InExtensionInstance->OnRemoved();
		{
			FScopeLock Lock(&GExtensionsCS);
			GExtensionInstances.FindChecked(InProfile.ToSharedPtr()).Remove(InExtensionInstance);
			GInstancesPerExtension.FindChecked(Extension.ToSharedPtr()).Remove(InExtensionInstance);
		}
	}

	void FLaunchExtension::DestroyExtensionInstancesForProfile(ILauncherProfilePtr InProfile)
	{
		if (!GExtensionsDestroyed && InProfile.IsValid())
		{
			RemoveExtensionInstancesForProfile(InProfile.ToSharedRef());
		}
	}

	TArray<TSharedRef<FLaunchExtensionInstance>> FLaunchExtension::GetExtensionInstances(TSharedRef<FLaunchExtension> InExtension, ILauncherProfileRef InProfile)
	{
		check(!GExtensionsDestroyed);

		FScopeLock Lock(&GExtensionsCS);

		TArray<TSharedRef<FLaunchExtensionInstance>> Result;
		
		const TArray<TSharedPtr<FLaunchExtensionInstance>>* InstancesForProfile = GExtensionInstances.Find(InProfile.ToSharedPtr());
		const TArray<TSharedPtr<FLaunchExtensionInstance>>* InstancesForExtension = GInstancesPerExtension.Find(InExtension.ToSharedPtr());
		if (InstancesForProfile != nullptr && InstancesForExtension != nullptr)
		{
			for (const TSharedPtr<FLaunchExtensionInstance>& ExtensionInstance : *InstancesForProfile)
			{
				if (InstancesForExtension->Contains(ExtensionInstance))
				{
					Result.Add(ExtensionInstance.ToSharedRef());
				}
			}
		}

		return MoveTemp(Result);
	}

	TSharedPtr<FLaunchExtensionInstance> FLaunchExtension::GetExtensionInstance(TSharedRef<FLaunchExtension> InExtension, ILauncherProfileRef InProfile)
	{
		if (InExtension->IsUATCommandManager())
		{
			return nullptr;
		}

		TArray<TSharedRef<FLaunchExtensionInstance>> ExtensionInstances = GetExtensionInstances(InExtension, InProfile);
		if (ExtensionInstances.Num() == 0)
		{
			return nullptr;
		}
		check(ExtensionInstances.Num() == 1);
		return ExtensionInstances[0];
	}


	void FLaunchExtension::GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const
	{
		MenuEntry = FExtensionsMenuEntry::Default;
	}

	FSlateIcon FLaunchExtension::GetIcon() const
	{
		return FSlateIcon( FProjectLauncherStyle::GetStyleSetName(), "Icons.Plugins" );
	}



	// common menu locations
	namespace
	{
		static const TCHAR* DefaultSectionName = TEXT("Common");
		static const FText DefaultSectionDisplayName = LOCTEXT("CommonSectionName","Common");

		static const TCHAR* AutoTestSectionName = TEXT("AutoTest");
		static const FText AutoTestSectionDisplayName = LOCTEXT("AutoTestSectionName", "Automated Tests");

		static const TCHAR* UATCmdsSectionName = TEXT("UATCmds");
		static const FText UATCmdsSectionDisplayName = LOCTEXT("UATCommandsSubMenu", "Additional Tasks");
	}


	const FLaunchExtension::FExtensionsMenuEntry FLaunchExtension::FExtensionsMenuEntry::None =
	{
		.Type = FLaunchExtension::FExtensionsMenuEntry::Type_None,
	};

	const FLaunchExtension::FExtensionsMenuEntry FLaunchExtension::FExtensionsMenuEntry::Default =
	{
		.SectionName = DefaultSectionName,
		.SectionDisplayName = DefaultSectionDisplayName,
		.Type = FLaunchExtension::FExtensionsMenuEntry::Type_Section,
		.bIsDefault = true,
	};

	const FLaunchExtension::FExtensionsMenuEntry FLaunchExtension::FExtensionsMenuEntry::OwnSection =
	{
		.Type = FLaunchExtension::FExtensionsMenuEntry::Type_Section,
	};

	const FLaunchExtension::FExtensionsMenuEntry FLaunchExtension::FExtensionsMenuEntry::Deprecated =
	{
		.SectionName = DefaultSectionName,
		.SubmenuName = TEXT("Deprecated"),
		.SectionDisplayName = DefaultSectionDisplayName,
		.SubmenuDisplayName = LOCTEXT("DeprecatedSubmenu", "Deprecated"),
		.Type = FLaunchExtension::FExtensionsMenuEntry::Type_SubMenu,
	};

	const FLaunchExtension::FExtensionsMenuEntry FLaunchExtension::FExtensionsMenuEntry::AutomatedTests =
	{
		.SectionName = AutoTestSectionName,
		.SectionDisplayName = AutoTestSectionDisplayName,
		.Type = FLaunchExtension::FExtensionsMenuEntry::Type_Section,
	};


	const FLaunchExtension::FExtensionsMenuEntry FLaunchExtension::FExtensionsMenuEntry::UATCommands =
	{
		.SectionName = UATCmdsSectionName,
		.SectionDisplayName = UATCmdsSectionDisplayName,
		.Type = FLaunchExtension::FExtensionsMenuEntry::Type_Section,
	};







	FLaunchExtensionInstance::FLaunchExtensionInstance( FArgs& InArgs )
		: Profile(InArgs.Profile)
		, DefaultProfile(InArgs.DefaultProfile)
		, Model(InArgs.Model)
		, Extension(InArgs.Extension)
		, OwnerTreeData(InArgs.OwnerTreeData)
	{
		Profile->OnCustomValidation().AddRaw( this, &FLaunchExtensionInstance::OnCustomValidation);
		Profile->OnProjectChanged().AddRaw( this, &FLaunchExtensionInstance::OnProjectChanged);
	}

	FLaunchExtensionInstance::~FLaunchExtensionInstance()
	{
		Profile->OnCustomValidation().RemoveAll(this);
		Profile->OnProjectChanged().RemoveAll(this);
	}

	void FLaunchExtensionInstance::InternalInitialize()
	{
	}

	void FLaunchExtensionInstance::CustomizeTree()
	{
		CustomizeTree(GetOwnerTreeData());
	}


	void FLaunchExtensionInstance::CustomizeTree( FLaunchProfileTreeData& ProfileTreeData )
	{
	}

	bool FLaunchExtensionInstance::GetExtensionVariables( TArray<FString>& OutVariables ) const
	{
		return false;
	}

	bool FLaunchExtensionInstance::GetExtensionVariableValue( const FString& InParameter, FString& OutValue ) const
	{
		return false;
	}

	FText FLaunchExtensionInstance::GetExtensionParameterDisplayName( const FString& InParameter ) const
	{
		return FText::FromString(InParameter);
	}

	bool FLaunchExtensionInstance::GetExtensionParameters( TArray<FString>& OutParameters ) const
	{
		return false;
	}

	FString FLaunchExtensionInstance::GetParameterValue( const FString& InParameter ) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CmdLineUtils::GetParameterValue(GetCommandLine(), InParameter);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	bool FLaunchExtensionInstance::UpdateParameterValue( const FString& InParameter, const FString& NewValue )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FString CommandLine = GetCommandLine();
		if (CmdLineUtils::UpdateParameterValue(CommandLine, InParameter, NewValue))
		{
			SetCommandLine(CommandLine);
			return true;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return false;
	}



	FString FLaunchExtensionInstance::GetFinalParameter( const FString& InParameter ) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CmdLineUtils::GetFinalParameter(GetCommandLine(), InParameter);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FLaunchExtensionInstance::AddParameter( const FString& InParameter )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FString CommandLine = GetCommandLine();
		CmdLineUtils::AddParameter(CommandLine, InParameter);
		SetCommandLine(CommandLine);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FLaunchExtensionInstance::RemoveParameter( const FString& InParameter )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FString CommandLine = GetCommandLine();
		if (CmdLineUtils::RemoveParameter(CommandLine, InParameter))
		{
			SetCommandLine(CommandLine);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}


	bool FLaunchExtensionInstance::IsParameterUsed( const FString& InParameter ) const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return CmdLineUtils::IsParameterUsed(GetCommandLine(), InParameter);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FLaunchExtensionInstance::SetParameterUsed( const FString& InParameter, bool bUsed )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FString CommandLine = GetCommandLine();
		CmdLineUtils::SetParameterUsed(CommandLine, InParameter, bUsed);
		SetCommandLine(CommandLine);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}



	FString FLaunchExtensionInstance::GetCommandLine() const
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Profile->GetAdditionalCommandLineParameters();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FLaunchExtensionInstance::SetCommandLine( const FString& CommandLine )
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Profile->SetAdditionalCommandLineParameters(CommandLine);
		BroadcastPropertyChanged();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FName FLaunchExtensionInstance::GetLaunchPlatformName() const
	{
		TArray<const PlatformInfo::FTargetPlatformInfo*> Infos;

		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
		{
			for (const PlatformInfo::FTargetPlatformInfo* Info : GetModel()->GetPlatformInfos(BuildCookRun))
			{
				Infos.AddUnique(Info);
			}
		}

		if (Infos.Num() == 0)
		{
			return NAME_None;
		}
		if (Infos.Num() == 1)
		{
			return Infos[0]->IniPlatformName;
		}

		auto FindType = [Infos]( EBuildTargetType Type )
		{
			for (const PlatformInfo::FTargetPlatformInfo* Info : Infos)
			{
				if (Info->PlatformType == Type)
				{
					return Info;
				}
			}

			return (const PlatformInfo::FTargetPlatformInfo*)nullptr;
		};

		const PlatformInfo::FTargetPlatformInfo* Result = FindType(EBuildTargetType::Game);
		Result = Result ? Result : FindType(EBuildTargetType::Client);
		Result = Result ? Result : FindType(EBuildTargetType::Server);
		Result = Result ? Result : FindType(EBuildTargetType::Editor);

		return Result ? Result->IniPlatformName : NAME_None;
	}

	FString FLaunchExtensionInstance::GetConfigString( EConfig Config, const TCHAR* Name, const TCHAR* DefaultValue ) const
	{
		FString KeyName = GetConfigKeyName(Config, Name);

		if (Config == EConfig::PerProfile)
		{
			const FString* ValuePtr = Profile->GetCustomStringProperties().Find( KeyName );
			return ValuePtr ? *ValuePtr : DefaultValue;
		}
		else
		{
			FString Value;
			return GConfig->GetString( Model->GetConfigSection() , *KeyName, Value, Model->GetConfigIni() ) ? Value : DefaultValue;
		}
	}

	bool FLaunchExtensionInstance::GetConfigBool( EConfig Config, const TCHAR* Name, bool DefaultValue ) const
	{
		FString KeyName = GetConfigKeyName(Config, Name);

		if (Config == EConfig::PerProfile)
		{
			const bool* ValuePtr = Profile->GetCustomBoolProperties().Find( KeyName );
			return ValuePtr ? *ValuePtr : DefaultValue;

		}
		else
		{
			bool Value;
			return GConfig->GetBool( Model->GetConfigSection() , *KeyName, Value, Model->GetConfigIni() ) ? Value : DefaultValue;
		}
	}

	int32 FLaunchExtensionInstance::GetConfigInteger( EConfig Config, const TCHAR* Name, int32 DefaultValue ) const
	{
		FString KeyName = GetConfigKeyName(Config, Name);

		if (Config == EConfig::PerProfile)
		{
			const FString* ValuePtr = Profile->GetCustomStringProperties().Find( KeyName );
			return ValuePtr ? FCString::Atoi(ValuePtr->GetCharArray().GetData()) : DefaultValue;

		}
		else
		{
			int32 Value;
			return GConfig->GetInt( Model->GetConfigSection() , *KeyName, Value, Model->GetConfigIni() ) ? Value : DefaultValue;
		}
	}

	float FLaunchExtensionInstance::GetConfigFloat( EConfig Config, const TCHAR* Name, float DefaultValue ) const
	{
		FString KeyName = GetConfigKeyName(Config, Name);

		if (Config == EConfig::PerProfile)
		{
			const FString* ValuePtr = Profile->GetCustomStringProperties().Find( KeyName );
			return ValuePtr ? FCString::Atof(ValuePtr->GetCharArray().GetData()) : DefaultValue;

		}
		else
		{
			float Value;
			return GConfig->GetFloat( Model->GetConfigSection() , *KeyName, Value, Model->GetConfigIni() ) ? Value : DefaultValue;
		}
	}


	void FLaunchExtensionInstance::SetConfigString( EConfig Config, const TCHAR* Name, const FString& Value ) const
	{
		FString KeyName = GetConfigKeyName(Config, Name);

		if (Config == EConfig::PerProfile)
		{
			Profile->GetCustomStringProperties().Add(KeyName, Value);
			BroadcastPropertyChanged();
		}
		else
		{
			GConfig->SetString( Model->GetConfigSection(), *KeyName, *Value, Model->GetConfigIni() );
		}
	}

	void FLaunchExtensionInstance::SetConfigBool( EConfig Config, const TCHAR* Name, const bool& Value ) const
	{
		FString KeyName = GetConfigKeyName(Config, Name);

		if (Config == EConfig::PerProfile)
		{
			Profile->GetCustomBoolProperties().Add(KeyName, Value);
			BroadcastPropertyChanged();
		}
		else
		{
			GConfig->SetBool( Model->GetConfigSection() , *KeyName, Value, Model->GetConfigIni() );
		}
	}

	void FLaunchExtensionInstance::SetConfigInteger( EConfig Config, const TCHAR* Name, const int32& Value ) const
	{
		FString KeyName = GetConfigKeyName(Config, Name);

		if (Config == EConfig::PerProfile)
		{
			Profile->GetCustomStringProperties().Add(KeyName, ::LexToString(Value) );
			BroadcastPropertyChanged();
		}
		else
		{
			GConfig->SetInt( Model->GetConfigSection() , *KeyName, Value, Model->GetConfigIni() );
		}
	}

	void FLaunchExtensionInstance::SetConfigFloat( EConfig Config, const TCHAR* Name, const float& Value ) const
	{
		FString KeyName = GetConfigKeyName(Config, Name);

		if (Config == EConfig::PerProfile)
		{
			Profile->GetCustomStringProperties().Add(KeyName, ::LexToString(Value) );
			BroadcastPropertyChanged();
		}
		else
		{
			GConfig->SetFloat( Model->GetConfigSection() , *KeyName, Value, Model->GetConfigIni() );
		}
	}

	FString FLaunchExtensionInstance::GetConfigKeyName( EConfig Config, const TCHAR* Name ) const
	{
		switch (Config)
		{
			case EConfig::PerProfile:
			case EConfig::User_Common:
			{
				return FString::Printf(TEXT("%s.%s"), GetInternalName(), Name );
			}

			case EConfig::User_PerProfile:
			{
				FGuid ProfileId = Profile->GetId();
				return FString::Printf(TEXT("%s.%s.%s"), GetInternalName(), *ProfileId.ToString(), Name );
			}
		}

		checkNoEntry();
		return Name;
	}

	void FLaunchExtensionInstance::BroadcastPropertyChanged() const
	{
		PropertyChangedDelegate.Broadcast();
	}

	void FLaunchExtensionInstance::RequestTreeRefresh()
	{
		OwnerTreeData.RequestTreeRefresh();
	}

	void FLaunchExtensionInstance::RequestFullTreeRebuild()
	{
		OwnerTreeData.RequestFullTreeRebuild();
	}

	FLaunchProfileTreeNode& FLaunchExtensionInstance::AddDefaultHeading(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData) const
	{
		return ProfileTreeData.GetHeading(FLaunchProfileTreeData::GeneralSettingsSectionName)
			.AddSubHeading(GetInternalName(), Extension->GetDisplayName(), INT32_MAX ); // new subheadings go at the end, after all of the properties in general settings
	}

	bool FLaunchExtensionInstance::CustomizeUATCommandLine( const ILauncherProfileUATCommandRef& InUATCommand, FString& InOutCommandLine )
	{
		return false;
	}

	void FLaunchExtensionInstance::OnPropertyChanged() 
	{
	}

	void FLaunchExtensionInstance::OnCustomValidation()
	{
		OnValidateProfile();
	}

	const TCHAR* FLaunchExtensionInstance::GetInternalName() const
	{
		return Extension->GetInternalName();
	}

	void FLaunchExtensionInstance::BroadcastEvent(const FString& EventName, void* EventData)
	{
		TArray<TSharedPtr<FLaunchExtensionInstance>> ExtensionInstances = GetProfileExtensionInstances(Profile);
		for (TSharedPtr<FLaunchExtensionInstance> ExtensionInstance : ExtensionInstances)
		{
			// Broadcast to all other extension instances associated with the same profile
			if (ExtensionInstance.IsValid() && ExtensionInstance.Get() != this && ExtensionInstance->GetProfile() == GetProfile())
			{
				ExtensionInstance->HandleEventCallback(FString::Printf(TEXT("%s_%s"), GetInternalName(), *EventName), EventData);
			}		
		}
	}

	TArray<TSharedPtr<FLaunchExtensionInstance>> FLaunchExtensionInstance::GetProfileExtensionInstancesByName(const FString& InternalName, ILauncherProfileRef InProfile)
	{
		TArray<TSharedPtr<FLaunchExtensionInstance>> ExtensionInstances = GetProfileExtensionInstances(InProfile);
		TArray<TSharedPtr<FLaunchExtensionInstance>> MatchedExtensionInstances;
		for (TSharedPtr<FLaunchExtensionInstance> ExtensionInstance : ExtensionInstances)
		{
			if (ExtensionInstance.IsValid())
			{
				TSharedRef<FLaunchExtension> Extension = ExtensionInstance->GetExtension();
				if (FString(Extension->GetInternalName()) == InternalName)
				{
					MatchedExtensionInstances.Add(ExtensionInstance);
				}
			}
		}
		return MatchedExtensionInstances;
	}
}

#undef LOCTEXT_NAMESPACE
