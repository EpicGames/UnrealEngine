// Copyright Epic Games, Inc. All Rights Reserved.

#include "Extension/UATCommandLaunchExtension.h"

#define LOCTEXT_NAMESPACE "FUATCommandLaunchExtensionInstance"


namespace ProjectLauncher
{
	FUATCommandLaunchExtensionInstanceBase::FUATCommandLaunchExtensionInstanceBase( FArgs& InArgs )
		: FLaunchExtensionInstance(InArgs)
		, UATCommand(InArgs.UATCommand.ToSharedRef())
	{
	}

	bool FUATCommandLaunchExtensionInstanceBase::ManagesUATCommand( const ILauncherProfileUATCommandRef& InUATCommand ) const
	{
		return UATCommand == InUATCommand;
	}

	ILauncherProfileUATCommandPtr FUATCommandLaunchExtensionInstanceBase::GetUATCommand() const
	{
		return UATCommand;
	}

	void FUATCommandLaunchExtensionInstanceBase::OnCustomValidation()
	{
		if (UATCommand->IsEnabled())
		{
			OnValidateProfile();
		}
	}


	bool FUATCommandLaunchExtensionInstanceBase::CustomizeUATCommandLine( const ILauncherProfileUATCommandRef& InUATCommand, FString& InOutCommandLine )
	{
		if (InUATCommand == UATCommand)
		{
			CustomizeUATCommandLine(InOutCommandLine);
			return true;
		}

		return false;
	}


	FLaunchProfileTreeNode& FUATCommandLaunchExtensionInstanceBase::AddDefaultHeading(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData) const
	{
		return ProfileTreeData.FindOrAddHeading(UATCommand, true);
	}

	const TCHAR* FUATCommandLaunchExtensionInstanceBase::GetInternalName() const
	{
		return UATCommand->GetInternalName();
	}



	TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FUATCommandLaunchExtensionBase::CreateUATCommandExtensionInstance(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel, FLaunchProfileTreeData& InOwnerTreeData)
	{
		ILauncherProfileUATCommandRef NewUATCommand = CreateUATCommand(InProfile, InModel);
		TSharedPtr<FLaunchExtensionInstance> ExtensionInstance = FLaunchExtension::CreateExtensionInstance(AsShared(), InProfile, InModel, InOwnerTreeData, NewUATCommand.ToSharedPtr());
		ExtensionInstance->OnAdded();

		return ExtensionInstance;
	}


}

#undef LOCTEXT_NAMESPACE
