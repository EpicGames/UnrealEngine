// Copyright Epic Games, Inc. All Rights Reserved.

#include "Model/ProjectLauncherModel.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigContext.h"
#include "Misc/App.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/ScopeLock.h"
#include "PlatformInfo.h"
#include "GameProjectHelper.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "ITargetDeviceServicesModule.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Dialog/SMessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Settings/ProjectPackagingSettings.h"
#include "Experimental/ZenServerInterface.h"
#include "HAL/PlatformProcess.h"
#include "Async/Async.h"

#if WITH_EDITOR
#include "Engine/World.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/AssetData.h"
#endif

#define LOCTEXT_NAMESPACE "SCustomLaunchProfileSelector"

namespace ProjectLauncher
{
	const FText ProjectLauncherText = LOCTEXT("ProjectLauncherTitle", "Project Launcher");
	bool bUseClassicView = false;

	const TCHAR* LastProfileKey = TEXT("LastProfileId");

	namespace CmdLineType
	{
		const TCHAR* Launch = TEXT("Launch");
		const TCHAR* UAT = TEXT("UAT");
	}

	FModel::FModel(const TSharedRef<ITargetDeviceProxyManager>& InDeviceProxyManager, const TSharedRef<ILauncher>& InLauncher, const TSharedRef<ILauncherProfileManager>& InProfileManager)
		: DeviceProxyManager(InDeviceProxyManager)
		, Launcher(InLauncher)
		, ProfileManager(InProfileManager)
	{
	}

	void FModel::Init()
	{
		// use a custom ini file so that it can be shared between UnrealFrontend and the editor
		ConfigFileName = FPaths::EngineSavedDir() / TEXT("Config") / TEXT("ProjectLauncher") / TEXT("UserSettings.ini");
		FConfigContext::ReadSingleIntoGConfig().Load(*ConfigFileName);
		LoadConfig();

		// register callbacks
		ProfileManager->OnPostProcessLaunchCommandLine().AddRaw( this, &FModel::OnModifyLaunchCommandLine );
		ProfileManager->OnPostProcessUATCommandLine().AddRaw( this, &FModel::OnModifyUATCommandLine );

		ProfileManager->OnProfileAdded().AddRaw(this, &FModel::HandleProfileManagerProfileAdded);
		ProfileManager->OnProfileRemoved().AddRaw(this, &FModel::HandleProfileManagerProfileRemoved);
		ProfileManager->OnProjectChanged().AddRaw(this, &FModel::HandleProfileManagerFallbackProjectChanged);

		DeviceProxyManager->OnProxyAdded().AddRaw(this, &FModel::HandleDeviceProxyAdded);
		DeviceProxyManager->OnProxyRemoved().AddRaw(this, &FModel::HandleDeviceProxyRemoved);

		ProjectSettingsReady.AddRaw(this, &FModel::HandleProjectSettingsReady);

		// ensure there's a project when we're in the editor
		// this means we don't need to display the global project selector in the editor
		if (GIsEditor && FPaths::IsProjectFilePathSet())
		{
			ProfileManager->SetProjectPath(FPaths::GetProjectFilePath());
		}
		GetProjectSettings(ProfileManager->GetProjectPath(), false); // begin caching the current project settings

		// prepare profiles
		BasicLaunchProfile = CreateBasicLaunchProfile();
		bHasSetBasicLaunchProfilePlatform = (BasicLaunchProfile->GetFirstBuildCookRun()->GetDeployedDeviceGroup()->GetDeviceIDs().Num() > 0);

		AllProfiles = ProfileManager->GetAllProfiles();
		for (const ILauncherProfilePtr& Profile : AllProfiles)
		{
			Profile->OnProjectChanged().AddRaw(this, &FModel::HandleProfileProjectChanged, Profile);

			Profile->SetBuildUAT(false); // this isn't serialized, so set a better default
		}

		AllProfiles.Add(BasicLaunchProfile);
		SortProfiles();

		DefaultBasicLaunchProfile = CreateBasicLaunchProfile();
		DefaultCustomLaunchProfile = CreateCustomProfile(TEXT("DefaultCustomProfile"));

		FString LastProfileId;
		if (GConfig->GetString(GetConfigSection(), LastProfileKey, LastProfileId, GetConfigIni()))
		{
			ILauncherProfilePtr LastProfile = ProfileManager->GetProfile(FGuid(LastProfileId));
			if (LastProfile.IsValid())
			{
				SelectProfile(LastProfile);
			}
		}
	}



	extern void ShutdownLaunchExtensions();

	FModel::~FModel()
	{
		for (TPair<FString, FProjectSettings>& Pair : CachedProjectSettings)
		{
			FPlatformProcess::ReturnSynchEventToPool(Pair.Value.Pending);
			delete Pair.Value.Config;
		}

		for (const ILauncherProfilePtr& Profile : AllProfiles)
		{
			Profile->OnProjectChanged().RemoveAll(this);
		}

		ProfileManager->OnPostProcessLaunchCommandLine().RemoveAll(this);
		ProfileManager->OnPostProcessUATCommandLine().RemoveAll(this);

		ProfileManager->OnProfileAdded().RemoveAll(this);
		ProfileManager->OnProfileRemoved().RemoveAll(this);
		ProfileManager->OnProjectChanged().RemoveAll(this);


		DeviceProxyManager->OnProxyAdded().RemoveAll(this);
		DeviceProxyManager->OnProxyRemoved().RemoveAll(this);

		SaveConfig();
		// ProfileManager->SaveSettings(); // not doing this at the moment because it also saves all the profiles which seems unnecessary

		ShutdownLaunchExtensions();
	}


	void FModel::SelectProfile(const ILauncherProfilePtr& NewProfile)
	{
		if (SelectedProfile != NewProfile)
		{
			ILauncherProfilePtr PreviousProfile = SelectedProfile;
			SelectedProfile = NewProfile;

			// try to make sure there is a valid build target
			if (SelectedProfile.IsValid())
			{
				for (const ILauncherProfileBuildCookRunRef& BuildCookRun : SelectedProfile->GetBuildCookRunCommands())
				{
					EnsureValidBuildTarget(SelectedProfile.ToSharedRef(), BuildCookRun);
				}

				GConfig->SetString(GetConfigSection(), LastProfileKey, *LexToString(SelectedProfile->GetId()), GetConfigIni());
			}

			ProfileSelectedDelegate.Broadcast(SelectedProfile, PreviousProfile);
		}
	}


	EProfileType FModel::GetProfileType(const ILauncherProfileRef& Profile) const
	{
		if (IsAdvancedProfile(Profile))
		{
			return EProfileType::Advanced;
		}
		else if (Profile == BasicLaunchProfile)
		{
			return EProfileType::Basic;
		}
		else
		{
			return EProfileType::Custom;
		}
	}

	bool FModel::IsAdvancedProfile(const ILauncherProfileRef& Profile) const
	{
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
		{
			if (BuildCookRun->GetPackagingMode() == ELauncherProfilePackagingModes::SharedRepository ||
				BuildCookRun->GetDeploymentMode() == ELauncherProfileDeploymentModes::CopyRepository ||
				BuildCookRun->GetLaunchMode() == ELauncherProfileLaunchModes::CustomRoles)
			{
				return true;
			}
		}

		return false;
	}




	const TCHAR* FModel::GetConfigSection() const
	{
		return TEXT("ProjectLauncher"); 
	}

	const FString& FModel::GetConfigIni() const
	{
		return ConfigFileName;
	}

	void FModel::LoadConfig()
	{
		if (!GIsEditor)
		{
			// restore the previous project selection
			FString ProjectPath;
	
			if (FPaths::IsProjectFilePathSet())
			{
				ProjectPath = FPaths::GetProjectFilePath();
			}
			else if (FGameProjectHelper::IsGameAvailable(FApp::GetProjectName()))
			{
				ProjectPath = FPaths::RootDir() / FApp::GetProjectName() / FApp::GetProjectName() + TEXT(".uproject");
			}
			else if (GConfig != nullptr)
			{
				GConfig->GetString(GetConfigSection(), TEXT("SelectedProjectPath"), ProjectPath, GetConfigIni());
			}
	
			ProfileManager->SetProjectPath(ProjectPath);
		}
	}



	void FModel::SaveConfig()
	{
		if (!GIsEditor && GConfig != nullptr && !FPaths::IsProjectFilePathSet() && !FGameProjectHelper::IsGameAvailable(FApp::GetProjectName()))
		{
			FString ProjectPath = ProfileManager->GetProjectPath();
			GConfig->SetString(GetConfigSection(), TEXT("SelectedProjectPath"), *ProjectPath, GetConfigIni());
		}
	}



	const PlatformInfo::FTargetPlatformInfo* FModel::GetPlatformInfo(const ILauncherProfilePtr& Profile)
	{
		if (Profile.IsValid() && Profile->GetFirstBuildCookRun() && Profile->GetFirstBuildCookRun()->GetCookedPlatforms().Num() > 0)
		{
			FString SelectedPlatform = Profile->GetFirstBuildCookRun()->GetCookedPlatforms()[0];
			return PlatformInfo::FindPlatformInfo(FName(SelectedPlatform));
		}

		return nullptr;
	}

	TArray<const PlatformInfo::FTargetPlatformInfo*> FModel::GetPlatformInfos(const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		TArray<const PlatformInfo::FTargetPlatformInfo*> Result;
		for (const FString& Platform : BuildCookRun->GetCookedPlatforms())
		{
			const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(Platform));
			if (PlatformInfo != nullptr)
			{
				Result.Add(PlatformInfo);
			}
		}

		return MoveTemp(Result);
	}

	TArray<const PlatformInfo::FTargetPlatformInfo*> FModel::GetPlatformInfosForAllCommands(const ILauncherProfilePtr& Profile)
	{
		TArray<const PlatformInfo::FTargetPlatformInfo*> Result;

		if (Profile.IsValid())
		{
			for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
			{
				for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : GetPlatformInfos(BuildCookRun))
				{
					Result.AddUnique(PlatformInfo);
				}
			}
		}

		return MoveTemp(Result);
	}


	const PlatformInfo::FTargetPlatformInfo* FModel::GetPlatformInfo(FName PlatformName, const FTargetInfo& BuildTargetInfo)
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformName);
		if (PlatformInfo == nullptr)
		{
			return nullptr;
		}

		// see if we found the platform immediately
		if (BuildTargetInfo.Name.IsEmpty() || PlatformInfo->PlatformType == BuildTargetInfo.Type)
		{
			return PlatformInfo;
		}

		// try to find a matching flavor for the given platform & build target
		if (PlatformInfo->VanillaInfo != nullptr)
		{
			if (PlatformInfo->VanillaInfo->PlatformType == BuildTargetInfo.Type && PlatformInfo->VanillaInfo->PlatformFlavor == PlatformInfo->PlatformFlavor)
			{
				return PlatformInfo->VanillaInfo;
			}
			for (const PlatformInfo::FTargetPlatformInfo* PlatformInfoFlavor : PlatformInfo->VanillaInfo->Flavors)
			{
				if (PlatformInfoFlavor->PlatformType == BuildTargetInfo.Type && PlatformInfoFlavor->PlatformFlavor == PlatformInfo->PlatformFlavor)
				{
					return PlatformInfoFlavor;
				}
			}
		}

		return nullptr;
	}




	bool FModel::IsHostPlatform(const ILauncherProfilePtr& Profile)
	{
		if (Profile.IsValid() && Profile->GetFirstBuildCookRun() && Profile->GetFirstBuildCookRun()->GetCookedPlatforms().Num() > 0)
		{
			FString SelectedPlatform = Profile->GetFirstBuildCookRun()->GetCookedPlatforms()[0];
			return IsHostPlatform(FName(SelectedPlatform));
		}

		return false;
	}

	bool FModel::IsHostPlatform(FName PlatformName)
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(PlatformName);

		if (PlatformInfo != nullptr && PlatformInfo->IniPlatformName == FPlatformProperties::IniPlatformName())
		{
			return true;
		}

		return false;
	}

	bool FModel::IsUsingRemotePlatform(const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		for (const FString& Platform : BuildCookRun->GetCookedPlatforms())
		{
			if (!IsHostPlatform(FName(Platform)))
			{
				return true;
			}
		}

		return false;
	}



	FTargetInfo FModel::GetBuildTargetInfo( FString BuildTargetName, const FString& ProjectPath )
	{
		if (BuildTargetName.IsEmpty())
		{
			BuildTargetName = GetProjectSettings(ProjectPath).DefaultBuildTargetName;
		}

		FTargetInfo Result;
		if (!BuildTargetName.IsEmpty())
		{
			const TArray<FTargetInfo>& BuildTargets = FDesktopPlatformModule::Get()->GetTargetsForProject(ProjectPath);
			for (const FTargetInfo& BuildTarget : BuildTargets)
			{
				if (BuildTarget.Name == BuildTargetName)
				{
					Result = BuildTarget;
					break;
				}
			}
		}

		return MoveTemp(Result);
	}

	FTargetInfo FModel::GetBuildTargetInfo( const ILauncherProfileRef& Profile )
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const FString& BuildTargetName = Profile->GetBuildTarget();
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		const FString& ProjectPath = Profile->GetProjectPath();
		return GetBuildTargetInfo(BuildTargetName, ProjectPath);
	}

	TArray<FTargetInfo> FModel::GetBuildTargetInfos(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		const FString& ProjectPath = Profile->GetProjectPath();
		const TArray<FString> BuildTargets = BuildCookRun->GetBuildTargets();

		TArray<FTargetInfo> Result;
		if (BuildTargets.Num() == 0 || BuildTargets.Contains(FString()))
		{
			Result.Add( GetBuildTargetInfo(FString(), ProjectPath) );
		}
		else for (const FString& BuildTarget : BuildTargets)
		{
			Result.Add( GetBuildTargetInfo(BuildTarget, ProjectPath) );
		}

		return MoveTemp(Result);
	}


	FString FModel::GetVanillaPlatformName( const FString& PlatformName )
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(*PlatformName);
		if (PlatformInfo != nullptr && PlatformInfo->VanillaInfo != nullptr)
		{
			return PlatformInfo->VanillaInfo->Name.ToString();
		}

		return PlatformName;
	}

	FString FModel::GetPlatformNameWithFlavor( const FString& PlatformName )
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(*PlatformName);
		if (PlatformInfo != nullptr && PlatformInfo->VanillaInfo != nullptr)
		{
			if (PlatformInfo->IsFlavor())
			{
				for (const PlatformInfo::FTargetPlatformInfo* PlatformFlavorInfo : PlatformInfo->VanillaInfo->Flavors)
				{
					if (PlatformFlavorInfo->PlatformFlavor == PlatformInfo->PlatformFlavor && PlatformFlavorInfo->PlatformType == EBuildTargetType::Game)
					{
						return PlatformFlavorInfo->Name.ToString();
					}
				}
			}
			else
			{
				return PlatformInfo->VanillaInfo->Name.ToString();
			}
		}

		return PlatformName;
	}

	FString FModel::GetBuildTargetPlatformName( const FString& PlatformName, const FTargetInfo& BuildTargetInfo )
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = ProjectLauncher::FModel::GetPlatformInfo(*PlatformName, BuildTargetInfo);
		if (PlatformInfo != nullptr)
		{
			return PlatformInfo->Name.ToString();
		}

		return PlatformName;

	}



	void FModel::SortProfiles()
	{
		AllProfiles.Sort( [this](const ILauncherProfilePtr& ProfileA, const ILauncherProfilePtr& ProfileB )
		{
			if (ProfileA == BasicLaunchProfile)
			{
				return true;
			}
			else if (ProfileB == BasicLaunchProfile)
			{
				return false;
			}
			else
			{
				return ProfileA->GetName() < ProfileB->GetName();
			}
		});
	}

	void FModel::HandleProfileManagerProfileAdded(const ILauncherProfileRef& Profile)
	{
		AllProfiles.Add(Profile);

		Profile->OnProjectChanged().AddRaw(this, &FModel::HandleProfileProjectChanged, Profile.ToSharedPtr());
	}

	extern void RemoveExtensionInstancesForProfile( const ILauncherProfileRef& Profile );


	void FModel::HandleProfileManagerProfileRemoved(const ILauncherProfileRef& Profile)
	{
		Profile->OnProjectChanged().RemoveAll(this);

		RemoveExtensionInstancesForProfile(Profile);

		AllProfiles.Remove(Profile);

		if (Profile == SelectedProfile)
		{
			SelectProfile(BasicLaunchProfile);
		}
	}

	void FModel::HandleProfileManagerFallbackProjectChanged()
	{
		if (SelectedProfile.IsValid() && !SelectedProfile->HasProjectSpecified() && AreProjectSettingsReady(SelectedProfile.ToSharedRef()))
		{
			for (const ILauncherProfileBuildCookRunRef& BuildCookRun : SelectedProfile->GetBuildCookRunCommands())
			{
				UpdateCookedPlatformsFromBuildTarget(SelectedProfile.ToSharedRef(), BuildCookRun);
			}
		}
	}

	void FModel::HandleProjectSettingsReady( const FString& ProjectPath )
	{
		if (ProjectPath == ProfileManager->GetProjectPath())
		{
			HandleProfileManagerFallbackProjectChanged();
		}
	}



	void FModel::HandleDeviceProxyAdded(const TSharedRef<ITargetDeviceProxy>& DeviceProxy )
	{
		if (!bHasSetBasicLaunchProfilePlatform && DeviceProxy->HasTargetPlatform(FPlatformProperties::IniPlatformName()) && ensure(BasicLaunchProfile->GetFirstBuildCookRun().IsValid()))
		{
			UpdatedCookedPlatformsFromDeployDeviceProxy(BasicLaunchProfile.ToSharedRef(), BasicLaunchProfile->GetFirstBuildCookRun().ToSharedRef(), DeviceProxy);
			bHasSetBasicLaunchProfilePlatform = (BasicLaunchProfile->GetFirstBuildCookRun()->GetDeployedDeviceGroup()->GetDeviceIDs().Num() > 0);
		}
	}

	void FModel::HandleDeviceProxyRemoved(const TSharedRef<ITargetDeviceProxy>& DeviceProxy )
	{
	}

	void FModel::HandleProfileProjectChanged( ILauncherProfilePtr InProfile )
	{
		if (InProfile.IsValid())
		{
			FString GlobalProjectPath;
			if (FPaths::IsProjectFilePathSet())
			{
				GlobalProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
			}

			FString ProjectPath = FPaths::ConvertRelativePathToFull(InProfile->GetProjectPath());
			if (!GlobalProjectPath.IsEmpty() && GlobalProjectPath == ProjectPath )
			{
				// this is the current project in-editor project - not for UFE. The build target list will have been loaded already and all ini files are already in memory
			}
			else
			{
				// begin caching the new project settings
				GetProjectSettings(InProfile->GetProjectPath(), false);
			}
		}
	}


	TSharedPtr<ITargetDeviceProxy> FModel::GetDeviceProxy(const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		ILauncherDeviceGroupPtr DeployedDeviceGroup = BuildCookRun->GetDeployedDeviceGroup();
		if (DeployedDeviceGroup != nullptr && DeployedDeviceGroup->GetDeviceIDs().Num() > 0)
		{
			const FString& DeviceID = DeployedDeviceGroup->GetDeviceIDs()[0];

			ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>("TargetDeviceServices");

			return TargetDeviceServicesModule.GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(DeviceID);
		}

		return nullptr;
	}


	ILauncherProfileRef FModel::CreateCustomProfile( const TCHAR* Name )
	{
		//create the profile
		ILauncherProfileRef Profile = ProfileManager->CreateUnsavedProfile(Name);

		//set defaults
		if (!ProfileManager->GetProjectPath().IsEmpty())
		{
			Profile->SetProjectPath(ProfileManager->GetProjectPath());
			Profile->SetProjectSpecified(true);
		}

		//new profiles should always have a default BuildCookRun set up
		check(Profile->GetFirstBuildCookRun().IsValid());
		ILauncherProfileBuildCookRunRef BuildCookRun = Profile->GetFirstBuildCookRun().ToSharedRef();
		SetDefaults(Profile, BuildCookRun);

		return Profile;
	}

	void FModel::SetDefaults(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		// set defaults
		FProjectSettings ProjectSettings = GetProjectSettings(Profile->GetProjectPath());
		if (ProjectSettings.bUseZenStore && ProjectSettings.bAllowRemoteNetworkService)
		{
			SetProfileContentScheme(EContentScheme::ZenStreaming, ProjectSettings, BuildCookRun );
		}
		else
		{
			SetProfileContentScheme(EContentScheme::PakFiles, ProjectSettings, BuildCookRun );
		}
		BuildCookRun->SetBuildConfiguration(EBuildConfiguration::Development);
		BuildCookRun->SetLaunchMode(ELauncherProfileLaunchModes::DefaultRole);
		BuildCookRun->SetBuildMode(ELauncherProfileBuildModes::Auto);
		// @fixme: set all defaults here 
		// ...


		// make sure there is a device & deploy group
		if (BuildCookRun->GetDeployedDeviceGroup() == nullptr)
		{
			ILauncherDeviceGroupRef DeployDeviceGroup = ProfileManager->AddNewDeviceGroup();
			BuildCookRun->SetDeployedDeviceGroup(DeployDeviceGroup);
		}

		TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxies;
		DeviceProxyManager->GetProxies(NAME_None, true, DeviceProxies);

		if (DeviceProxies.Num() > 0 && DeviceProxies[0].IsValid()) // item 0 should always be the host platform
		{
			UpdatedCookedPlatformsFromDeployDeviceProxy(Profile, BuildCookRun, DeviceProxies[0].ToSharedRef());
		}
		else
		{
			BuildCookRun->AddCookedPlatform(FPlatformProperties::IniPlatformName());
		}

		EnsureValidBuildTarget(Profile, BuildCookRun);
	}

	ILauncherProfileRef FModel::CreateBasicLaunchProfile()
	{
		const FText BasicLaunchProfileName = LOCTEXT("BasicLaunchProfileName", "Basic Launch");
		const FText BasicLaunchProfileDescription = LOCTEXT("BasicLaunchProfileDescription", "Use this profile to launch on a device with the recommended defaults");

		ILauncherProfileRef Profile = CreateCustomProfile(*BasicLaunchProfileName.ToString());
		Profile->SetDescription(*BasicLaunchProfileDescription.ToString());
		Profile->SetProjectSpecified(false);
		Profile->GetFirstBuildCookRun()->SetBuildTargetSpecified(false);

		return Profile;
	}


	void FModel::UpdateDeployDeviceProxyFromCookedPlatforms(const ILauncherProfileRef& Profile)
	{
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
		{
			ILauncherDeviceGroupPtr DeviceGroup = BuildCookRun->GetDeployedDeviceGroup();
			if (DeviceGroup.IsValid())
			{
				TSet<FString> DeviceIDs( DeviceGroup->GetDeviceIDs() );

				TArray<FString> Platforms = BuildCookRun->GetCookedPlatforms();

				TSet<FString> NewDeviceIDs;
				for ( const FString& DeviceID : DeviceIDs)
				{
					TSharedPtr<ITargetDeviceProxy> DeviceProxy = DeviceProxyManager->FindProxyDeviceForTargetDevice(DeviceID);
					if (DeviceProxy.IsValid())
					{
						TArray<FName> Variants;
						DeviceProxy->GetVariants(Variants);

						for (const FString& Platform : Platforms)
						{
							for( const FName& Variant : Variants)
							{
								FString TargetPlatformName = DeviceProxy->GetTargetPlatformName(Variant);
								if (Platform == TargetPlatformName)
								{
									const TSet<FString>& TargetDeviceIds = DeviceProxy->GetTargetDeviceIds(Variant);
									NewDeviceIDs.Append(TargetDeviceIds);
								}
							}
						}
					}
					else
					{
						NewDeviceIDs.Add(DeviceID);
					}
				}

				// update the device group if there's been changes
				if (NewDeviceIDs.Difference(DeviceIDs).Num() > 0 || DeviceIDs.Difference(NewDeviceIDs).Num() > 0)
				{
					DeviceGroup->RemoveAllDevices();

					for (const FString& NewDeviceID : NewDeviceIDs)
					{
						DeviceGroup->AddDevice(NewDeviceID);
					}
				}
			}
		}
	}

	void FModel::UpdatedCookedPlatformsFromDeployDeviceProxy(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun, TSharedPtr<ITargetDeviceProxy> DeviceProxy)
	{
		if (DeviceProxy.IsValid())
		{
			BuildCookRun->GetDeployedDeviceGroup()->RemoveAllDevices();
			BuildCookRun->GetDeployedDeviceGroup()->AddDevice(DeviceProxy->GetTargetDeviceId(NAME_None));
		}
		else if (BuildCookRun->GetDeployedDeviceGroup()->GetNumDevices() > 0)
		{
			DeviceProxy = GetDeviceProxy(BuildCookRun);
		}
		
		if (DeviceProxy.IsValid())
		{
			BuildCookRun->ClearCookedPlatforms();

			TArray<FTargetInfo> BuildTargetInfos = GetBuildTargetInfos(Profile, BuildCookRun);
			for (const FTargetInfo& BuildTargetInfo : BuildTargetInfos)
			{
				FString PlatformName = DeviceProxy->GetTargetPlatformName(NAME_None);
				const PlatformInfo::FTargetPlatformInfo* PlatformInfo = GetPlatformInfo(*PlatformName, BuildTargetInfo);

				if (PlatformInfo != nullptr)
				{
					BuildCookRun->AddCookedPlatform(PlatformInfo->Name.ToString());
				}
				else
				{
					BuildCookRun->AddCookedPlatform(DeviceProxy->GetTargetPlatformName(NAME_None));
				}
			}
		}
	}

	void FModel::UpdateCookedPlatformsFromBuildTarget(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		TArray<FTargetInfo> BuildTargetInfos = GetBuildTargetInfos(Profile, BuildCookRun);

		if (BuildTargetInfos.Num() != 1)
		{
			return;
		}
		const FTargetInfo& BuildTargetInfo = BuildTargetInfos[0];

		TArray<FString> Platforms = BuildCookRun->GetCookedPlatforms();
		BuildCookRun->ClearCookedPlatforms();
		for (const FString& Platform : Platforms)
		{
			const PlatformInfo::FTargetPlatformInfo* PlatformInfo = GetPlatformInfo(*Platform, BuildTargetInfo);
			if (PlatformInfo != nullptr)
			{
				BuildCookRun->AddCookedPlatform(PlatformInfo->Name.ToString());
			}
			else
			{
				BuildCookRun->AddCookedPlatform(Platform);
			}
		}
	}


	void FModel::EnsureValidBuildTarget(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		if (Profile->HasProjectSpecified() && AreProjectSettingsReady(Profile))
		{
			const FProjectSettings& ProjectSettings = GetProjectSettings(Profile);
			if (!ProjectSettings.FallbackDefaultBuildTarget.IsEmpty() && (BuildCookRun->GetBuildTargets().Num() == 0 || BuildCookRun->GetBuildTargets().Contains(FString())))
			{
				BuildCookRun->ClearBuildTargets();
				BuildCookRun->AddBuildTarget(ProjectSettings.FallbackDefaultBuildTarget);
				BuildCookRun->SetBuildTargetSpecified(true);

				UpdateCookedPlatformsFromBuildTarget(Profile, BuildCookRun);
			}
		}
	}


	ILauncherProfilePtr FModel::CloneCustomProfile(const ILauncherProfileRef& Profile)
	{
		// capture the current profile
		FString ExistingProfileJson;
		TSharedRef<TJsonWriter<>> JsonWriter = TJsonWriterFactory<>::Create(&ExistingProfileJson);
		Profile->Save(JsonWriter.Get());
		JsonWriter->Close();

		// create a new json object
		TSharedPtr<FJsonObject> ProfileJsonObject;
		TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(ExistingProfileJson);
		if (FJsonSerializer::Deserialize(JsonReader, ProfileJsonObject) && ProfileJsonObject.IsValid())
		{
			// create the cloned profile & load the json
			ILauncherProfilePtr NewProfile = CreateCustomProfile(TEXT("Cloned"));
			if (NewProfile.IsValid() && NewProfile->Load(*ProfileJsonObject.Get()))
			{
				// need to give it a new Id - don't use the cloned one!
				NewProfile->AssignId(true);

				// create a new device group because these are not serialized
				for (ILauncherProfileBuildCookRunRef NewBuildCookRun : NewProfile->GetBuildCookRunCommands())
				{
					ILauncherDeviceGroupRef DeployDeviceGroup = ProfileManager->AddNewDeviceGroup();
					NewBuildCookRun->SetDeployedDeviceGroup(DeployDeviceGroup);

					ILauncherProfileBuildCookRunPtr BuildCookRun = Profile->GetBuildCookRunCommand(NewBuildCookRun->GetInternalName());
					if (BuildCookRun.IsValid())
					{
						for (const FString& DeviceID : BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs())
						{
							DeployDeviceGroup->AddDevice(DeviceID);
						}
					}
				}

				return NewProfile;
			}
		}

		return nullptr;
	}

			

	EContentScheme FModel::DetermineProfileContentScheme(const ILauncherProfileBuildCookRunRef& BuildCookRun) const
	{
		if (BuildCookRun->GetCookMode() == ELauncherProfileCookModes::OnTheFly)
		{
			return EContentScheme::CookOnTheFly;
		}
		else if (BuildCookRun->IsUsingPreStagedBuild())
		{
			return EContentScheme::PreStagedBuild;
		}
		else if (BuildCookRun->GetPackagingMode() != ELauncherProfilePackagingModes::DoNotPackage)
		{
			if (BuildCookRun->IsCreatingReleaseVersion() || BuildCookRun->IsGeneratingPatch() || BuildCookRun->IsCreatingDLC())
			{
				return EContentScheme::SubmissionPackage;
			}
			else
			{
				return EContentScheme::DevelopmentPackage;
			}
		}
		else if (BuildCookRun->IsPackingWithUnrealPak())
		{
			return EContentScheme::PakFiles;
		}
		else if (BuildCookRun->IsUsingZenPakStreaming())
		{
			return EContentScheme::ZenPakStreaming;
		}
		else if (BuildCookRun->IsUsingZenStore())
		{
			return EContentScheme::ZenStreaming;
		}
		else
		{
			return EContentScheme::LooseFiles;
		}
	}



	void FModel::SetProfileContentScheme(EContentScheme ContentScheme, const FProjectSettings& ProjectSettings,
		const ILauncherProfileBuildCookRunRef& BuildCookRun, bool bWantToCook,
		ELauncherProfileDeploymentModes::Type DefaultDeploymentMode)
	{
		bool bPakFiles = (ContentScheme == EContentScheme::PakFiles || ContentScheme == EContentScheme::DevelopmentPackage || ContentScheme == EContentScheme::SubmissionPackage);
		bool bUseZen = (ContentScheme == EContentScheme::ZenStreaming || ContentScheme == EContentScheme::ZenPakStreaming) ||
			(ContentScheme != EContentScheme::LooseFiles && ProjectSettings.bUseZenStore);
		bool bUseZenStreaming = (ContentScheme == EContentScheme::ZenStreaming);
		bool bCOTF = (ContentScheme == EContentScheme::CookOnTheFly);
		bool bPackage = (ContentScheme == EContentScheme::DevelopmentPackage || ContentScheme == EContentScheme::SubmissionPackage);
		bool bZenPakStreaming = (ContentScheme == EContentScheme::ZenPakStreaming);
		bool bSubmissionPackage = (ContentScheme == EContentScheme::SubmissionPackage);
		bool bPreStagedBuild = (ContentScheme == EContentScheme::PreStagedBuild);

		// FIXME: If !bUseZen but ProjectSettings.bUseZenStore, we should pass -skipzenstore into the cooker. There is
		// not currently an argument in UAT to pass that in, though. In the meantime we just don't support that selection,
		// and grey out the ability to select EContentScheme::LooseFiles if !bUseZenStore.

		BuildCookRun->SetUseZenStreaming(bUseZenStreaming);
		BuildCookRun->SetUsePreStagedBuild(bPreStagedBuild);
		BuildCookRun->SetUseZenPakStreaming(bZenPakStreaming);
		BuildCookRun->SetDeployWithUnrealPak(bPakFiles);
		BuildCookRun->SetUseZenStore(bUseZen);
		BuildCookRun->SetCreateReleaseVersion(bSubmissionPackage);
		BuildCookRun->SetGeneratePatch(false);
		BuildCookRun->SetCreateDLC(false);
		if (!bPakFiles)
		{
			BuildCookRun->SetGenerateChunks(false);
			BuildCookRun->SetUseIoStore(false);
		}

		ELauncherProfileDeploymentModes::Type DeploymentMode = DefaultDeploymentMode;
		if (bCOTF)
		{
			BuildCookRun->SetCookMode(ELauncherProfileCookModes::OnTheFly);
			DeploymentMode = ELauncherProfileDeploymentModes::FileServer;
		}
		else if (bZenPakStreaming || !bWantToCook)
		{
			BuildCookRun->SetCookMode(ELauncherProfileCookModes::DoNotCook);
		}
		else
		{
			BuildCookRun->SetCookMode(ELauncherProfileCookModes::ByTheBook);
		}

		if (bPackage)
		{
			BuildCookRun->SetDeploymentMode(ELauncherProfileDeploymentModes::DoNotDeploy); // @todo: some platforms support package deployment. should update verification to check new function in ITargetPlatformControls
			BuildCookRun->SetPackagingMode(ELauncherProfilePackagingModes::Locally);
		}
		else
		{
			BuildCookRun->SetDeploymentMode(DeploymentMode);
			BuildCookRun->SetPackagingMode(ELauncherProfilePackagingModes::DoNotPackage);
		}
	}


	void FModel::ReadProjectSettingsFromConfig( FConfigCacheIni& InConfig, const FString& InProjectPath, FProjectSettings& OutResult )
	{
		// read build settings
		const TCHAR* BuildSettings = TEXT("/Script/BuildSettings.BuildSettings");
		InConfig.GetString(BuildSettings, TEXT("DefaultEditorTarget"), OutResult.DefaultEditorTargetName, GEngineIni);
		InConfig.GetString(BuildSettings, TEXT("DefaultServerTarget"), OutResult.DefaultServerTargetName, GEngineIni);
		InConfig.GetString(BuildSettings, TEXT("DefaultClientTarget"), OutResult.DefaultClientTargetName, GEngineIni);

		// read project packaging settings
		const TCHAR* ProjectPackagingConfigSection = TEXT("/Script/UnrealEd.ProjectPackagingSettings");
		InConfig.GetBool(ProjectPackagingConfigSection, TEXT("bUseZenStore"), OutResult.bUseZenStore, GGameIni);
		InConfig.GetBool(ProjectPackagingConfigSection, TEXT("bUseIoStore"), OutResult.bUseIoStore, GGameIni);
		InConfig.GetBool(ProjectPackagingConfigSection, TEXT("bEnablePakStreaming"), OutResult.bHasAutomaticZenPakStreamingWorkspaceCreation, GGameIni);
		InConfig.GetString(ProjectPackagingConfigSection, TEXT("BuildTarget"), OutResult.DefaultBuildTargetName, GGameIni);

		if (OutResult.DefaultBuildTargetName.IsEmpty())
		{
			const TArray<FTargetInfo>& BuildTargets = FDesktopPlatformModule::Get()->GetTargetsForProject(*InProjectPath);
			const TArray<FTargetInfo> GameBuildTargets = BuildTargets.FilterByPredicate([](const FTargetInfo& TargetInfo)
			{
				return TargetInfo.Type == EBuildTargetType::Game;
			});
			if (GameBuildTargets.Num() == 1)
			{
				OutResult.DefaultBuildTargetName = GameBuildTargets[0].Name;
			}
			// multiple (or no) game targets... would need to read the BuildTarget from /Script/BuildSettings.BuildSettings... also reading the platform-specific ini
			// ... fixme ...

			// find a fallback build target
			// pick the first Game build target... if there are no Game build targets available, default to a Client build target if we can
			if (OutResult.DefaultBuildTargetName.IsEmpty())
			{
				if (GameBuildTargets.Num() > 0)
				{
					OutResult.FallbackDefaultBuildTarget = GameBuildTargets[0].Name;
				}
				else if (!OutResult.DefaultClientTargetName.IsEmpty())
				{
					OutResult.FallbackDefaultBuildTarget = OutResult.DefaultClientTargetName;
				}
				else
				{
					const TArray<FTargetInfo> ClientBuildTargets = BuildTargets.FilterByPredicate([](const FTargetInfo& TargetInfo)
					{
						return TargetInfo.Type == EBuildTargetType::Client;
					});
					if (ClientBuildTargets.Num() > 0)
					{
						OutResult.FallbackDefaultBuildTarget = ClientBuildTargets[0].Name;
					}
				}
			}

		}
		
		// read zen settings
		using namespace UE::Zen;
		const FServiceSettings& ZenServiceSettings = GetDefaultServiceInstance().GetServiceSettings();
		OutResult.bAllowRemoteNetworkService = ZenServiceSettings.IsAutoLaunch() && ZenServiceSettings.GetRemoteNetworkService() != ERemoteNetworkService::None;
	}
	
	
	bool FModel::AreProjectSettingsReady(const FString& ProjectPath)
	{
		FScopeLock Lock(&CachedProjectSettingsCS);
		return GetProjectSettings(ProjectPath, false).bIsReady;
	}

	bool FModel::AreProjectSettingsReady(const ILauncherProfileRef& Profile)
	{
		return AreProjectSettingsReady(*Profile->GetProjectPath());
	}
	
	

	const FProjectSettings FModel::GetProjectSettings( const FString& InProjectPath, bool bWait )
	{
		FProjectSettings Result;
		Result.bIsReady = true;

		FString ProjectPath = FPaths::ConvertRelativePathToFull(InProjectPath);
		FString ProjectName = FPaths::GetBaseFilename(ProjectPath);

		FString GlobalProjectPath;
		if (FPaths::IsProjectFilePathSet())
		{
			GlobalProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
		}

		if (!GlobalProjectPath.IsEmpty() && GlobalProjectPath == ProjectPath )
		{
			// read the current project's ini settings
			Result.bIsCurrentEditorProject = true;
			Result.Config = GConfig;
			ReadProjectSettingsFromConfig(*GConfig, InProjectPath, Result);

			// apply any in-memory properties @fixme: is this necessary?
			const UProjectPackagingSettings* ProjectPackagingSettings = UProjectPackagingSettings::StaticClass()->GetDefaultObject<UProjectPackagingSettings>();
			Result.bUseZenStore = ProjectPackagingSettings->bUseZenStore;
			Result.bUseIoStore = ProjectPackagingSettings->bUseIoStore;
			if (!ProjectPackagingSettings->BuildTarget.IsEmpty())
			{
				Result.DefaultBuildTargetName = ProjectPackagingSettings->BuildTarget;
			}

			// note this is not cached because the user could edit the properties from within the editor
		}
		else if (!ProjectName.IsEmpty())
		{
			FEvent* WaitEvent = nullptr;

			// find or add cached project settings
			{
				FScopeLock Lock(&CachedProjectSettingsCS);

				// check if it exists already
				if (!CachedProjectSettings.Contains(ProjectName))
				{
					// create a placeholder entry
					FProjectSettings& Settings = CachedProjectSettings.Emplace( ProjectName );
					Settings.Pending = FPlatformProcess::GetSynchEventFromPool(true);
					WaitEvent = Settings.Pending;

					// initialize it in the background
					AsyncTask(ENamedThreads::AnyHiPriThreadNormalTask, [WeakThis = AsWeak(), ProjectName, ProjectPath = InProjectPath]()
					{
						TSharedPtr<FModel> StrongThis = WeakThis.Pin();
						if (StrongThis.IsValid())
						{
							FProjectSettings NewSettings;

							// load the other project's ini files into a temporary config cache
							NewSettings.Config = new FConfigCacheIni(EConfigCacheType::Temporary);
							FConfigContext Context = FConfigContext::ReadIntoConfigSystem(NewSettings.Config, FString());
							Context.ProjectConfigDir = FPaths::Combine(FPaths::GetPath(ProjectPath), TEXT("Config/"));

							const TCHAR* IniNames[] = { *GEngineIni, *GGameIni }; // only need a subset of the other project's config files
							for (const TCHAR* IniName : IniNames)
							{
								Context.Load(IniName);
							}

							// read the ini settings
							NewSettings.bIsCurrentEditorProject = false;
							StrongThis->ReadProjectSettingsFromConfig(*NewSettings.Config, ProjectPath, NewSettings);

							// cache the result
							{
								FScopeLock Lock(&StrongThis->CachedProjectSettingsCS);
								NewSettings.Pending = StrongThis->CachedProjectSettings[ProjectName].Pending;
								StrongThis->CachedProjectSettings[ProjectName] = MoveTemp(NewSettings);
								StrongThis->CachedProjectSettings[ProjectName].bIsReady = true;
							}

							// signal completion
							StrongThis->CachedProjectSettings[ProjectName].Pending->Trigger();
							ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakThis, ProjectPath]()
							{
								TSharedPtr<FModel> StrongThis = WeakThis.Pin();
								if (StrongThis.IsValid())
								{
									StrongThis->ProjectSettingsReady.Broadcast(ProjectPath);
								}
							});
						}
					});
				}
			
				WaitEvent = CachedProjectSettings[ProjectName].Pending;				
			}

			// optionally wait for the settings to be ready
			if (bWait && WaitEvent != nullptr)
			{
				WaitEvent->Wait();
			}

			// grab the cached result
			{
				FScopeLock Lock(&CachedProjectSettingsCS);
				Result = CachedProjectSettings[ProjectName];
			}
		}

		return MoveTemp(Result);
	}
	
	
	
	
	const FProjectSettings FModel::GetProjectSettings( const ILauncherProfileRef& Profile, bool bWait )
	{
		return GetProjectSettings(*Profile->GetProjectPath(), bWait);
	}

	FString FModel::GetProjectPath(const ILauncherProfileRef& Profile) const
	{
		if (IsBasicLaunchProfile(Profile))
		{
			if (Profile->HasProjectSpecified())
			{
				return Profile->GetProjectPath();
			}
			else
			{
				return GetProfileManager()->GetProjectPath();
			}
		}
		else
		{
			if (Profile->HasProjectSpecified())
			{
				return Profile->GetProjectPath();
			}

			return FString();
		}

	}

	bool FModel::IsBasicLaunchProfile( const ILauncherProfilePtr& Profile ) const
	{
		return (Profile == BasicLaunchProfile);
	}


	const ILauncherProfileRef FModel::GetDefaultBasicLaunchProfile() const
	{
		return DefaultBasicLaunchProfile.ToSharedRef();
	}

	const ILauncherProfileRef FModel::GetDefaultCustomLaunchProfile() const
	{
		return DefaultCustomLaunchProfile.ToSharedRef();
	}


	const ILauncherProfileBuildCookRunRef FModel::GetBasicDefaultBuildCookRun() const
	{
		return DefaultBasicLaunchProfile->GetFirstBuildCookRun().ToSharedRef();
	}

	const ILauncherProfileBuildCookRunRef FModel::GetCustomDefaultBuildCookRun() const
	{
		return DefaultCustomLaunchProfile->GetFirstBuildCookRun().ToSharedRef();
	}


	TSharedPtr<FLaunchLogMessage> FModel::AddLogMessage( const FString& InMessage, ELogVerbosity::Type InVerbosity )
	{
		TSharedPtr<FLaunchLogMessage> Message = MakeShared<FLaunchLogMessage>(InMessage, InVerbosity);
		LaunchLogMessages.Add(Message);
		return Message;
	}

	void FModel::ClearLogMessages()
	{
		LaunchLogMessages.Reset();
	}



	extern void ApplyExtensionVariables( const ILauncherProfileRef& InProfile, FString& InOutCommandLine, TSharedRef<FModel> InModel, EBuildTargetType InBuildTargetType );

	void FModel::OnModifyLaunchCommandLine( const ILauncherProfileRef& InProfile, EBuildTargetType InBuildTargetType, FString& InOutCommandLine )
	{
		ApplyExtensionVariables(InProfile, InOutCommandLine, AsShared(), InBuildTargetType );
	}


	extern void ApplyExtensionVariablesForUATCommand( FUATCommandPostProcessParameters& PostProcessParameters, TSharedRef<FModel> Model );

	void FModel::OnModifyUATCommandLine( FUATCommandPostProcessParameters& PostProcessParameters )
	{
		ApplyExtensionVariablesForUATCommand(PostProcessParameters, AsShared());
	}

	bool FModel::AreExtensionsEnabled() const
	{
#if WITH_EDITOR
		// make sure the extensions system is enabled
		bool bEnableProjectLauncherExtensions = false;
		GConfig->GetBool(TEXT("/Script/UnrealEd.EditorExperimentalSettings"), TEXT("bEnableProjectLauncherExtensions"), bEnableProjectLauncherExtensions, GEditorPerProjectIni);

		return bEnableProjectLauncherExtensions;
#else

		// The ini file is not loaded by UnrealFrontend because its per-project, so allow extensions by default outside of the editor.
		return true;
#endif
	}

	bool FModel::CanUseSimplifiedLayout(const ILauncherProfileRef& Profile)
	{
		if (IsBasicLaunchProfile(Profile))
		{
			return true;
		}

		if (!ProjectLauncher::bUseClassicView)
		{
			return false;
		}

		return (Profile->GetBuildCookRunCommands().Num() == 1 && Profile->GetAutomatedTests().Num() == 0 && Profile->GetUATCommands().Num() == 1);
	}

	TArray<FString> FModel::GetAndCacheMapPaths(const FString& InContentDir, bool bIncludeNonContentDirMaps) // @todo: map list parsing should ideally be asyncronous, showing a spinner in the map selector controls until its finished etc.
	{
		// the editor has access to the asset registry which is much faster and will be up to date if new maps are added at runtime
#if WITH_EDITOR
		if (FPaths::IsProjectFilePathSet())
		{
			FString GlobalProjectPath = FPaths::ConvertRelativePathToFull(FPaths::GetPath(FPaths::GetProjectFilePath()));
			FString EngineDir = FPaths::ConvertRelativePathToFull(FPaths::EngineDir());
			FString ContentDirFull = FPaths::ConvertRelativePathToFull(InContentDir);

			// only use the asset registry when the content directory is under the current project or engine tree
			// (the asset registry only indexes content from the currently open project, engine, and their plugins)
			if (!GlobalProjectPath.IsEmpty() && (FPaths::IsUnderDirectory(ContentDirFull, GlobalProjectPath) || FPaths::IsUnderDirectory(ContentDirFull, EngineDir) || bIncludeNonContentDirMaps))
			{
				// gather all world asset metadata
				FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				TArray<FAssetData> MapAssets;
				AssetRegistryModule.Get().GetAssetsByClass(UWorld::StaticClass()->GetClassPathName(), MapAssets, true);

				// build the list of map file names for the maps in the desired content directory
				TArray<FString> MapFileList;
				for (const FAssetData& MapAsset : MapAssets)
				{
					FString MapFileName = FPackageName::LongPackageNameToFilename(MapAsset.PackageName.ToString(), FPackageName::GetMapPackageExtension());
					MapFileName = FPaths::ConvertRelativePathToFull(MapFileName);

					if (FPaths::IsUnderDirectory(MapFileName, ContentDirFull) || bIncludeNonContentDirMaps)
					{
						MapFileList.Add( MapFileName );
					}
				}

				MapFileList.Sort();
				return MapFileList;
			}
		}
#endif

		// check to see if we've cached these maps already
		const TArray<FString>* CachedMapListPtr = CachedMapPaths.Find(InContentDir);
		if (CachedMapListPtr != nullptr)
		{
			return *CachedMapListPtr;
		}

		// this is slow & currently blocking so show a wait dialog. 
		// annoyingly this doesn't show up in UnrealFrontend because that uses FFeedbackContext rather than FFeedbackContextEditor
		FScopedSlowTask SlowTask(0, LOCTEXT("CacheProjectMapsDesc","Caching project maps"));
		SlowTask.MakeDialog();

		// search for map files
		TArray<FString> MapFileList;
		const FString WildCard = FString::Printf(TEXT("*%s"), *FPackageName::GetMapPackageExtension());
		IFileManager::Get().FindFilesRecursive(MapFileList, *InContentDir, *WildCard, true, false);

		MapFileList.Sort();
		CachedMapPaths.Add(InContentDir, MapFileList);
		return MapFileList;
	}

	TArray<FString> FModel::GetAvailableProjectMapNames( const FString& InProjectPath, bool bIncludeNonContentDirMaps)
	{
		TArray<FString> Maps = GetAvailableProjectMapPaths(InProjectPath, bIncludeNonContentDirMaps);
		for (FString& Map : Maps)
		{
			Map = FPaths::GetBaseFilename(Map);
		}
		return MoveTemp(Maps);
	}

	TArray<FString> FModel::GetAvailableProjectMapPaths( const FString& InProjectPath, bool bIncludeNonContentDirMaps)
	{
		const FString ProjectPath = FPaths::IsRelative(InProjectPath) ? FPaths::Combine(FPaths::RootDir(), InProjectPath) : InProjectPath;
		const FString ContentDir = FPaths::Combine(ProjectPath, TEXT("Content"));
		return GetAndCacheMapPaths(ContentDir, bIncludeNonContentDirMaps);
	}

	TArray<FString> FModel::GetAvailableEngineMapNames()
	{
		TArray<FString> Maps = GetAvailableEngineMapPaths();
		for (FString& Map : Maps)
		{
			Map = FPaths::GetBaseFilename(Map);
		}
		return MoveTemp(Maps);

	}

	TArray<FString> FModel::GetAvailableEngineMapPaths()
	{
		return GetAndCacheMapPaths(FPaths::Combine(FPaths::EngineContentDir(), TEXT("Maps")));
	}

	TArray<FString> FModel::GetAvailablePluginMapNames(const FString& PluginContentDir)
	{
		TArray<FString> Maps = GetAvailablePluginMapPaths(PluginContentDir);
		for (FString& Map : Maps)
		{
			Map = FPaths::GetBaseFilename(Map);
		}
		return MoveTemp(Maps);
	}

	TArray<FString> FModel::GetAvailablePluginMapPaths(const FString& PluginContentDir)
	{
		return GetAndCacheMapPaths(PluginContentDir);
	}

	void FModel::GetUATCommandsDetail( const ILauncherProfileRef& Profile, FUATCommandsDetail& OutUATCommandsDetail)
	{
		const TArray<ILauncherProfileUATCommandRef>& UATCommands = Profile->GetUATCommands();
		OutUATCommandsDetail.NumUATCommands = UATCommands.Num();
		if (OutUATCommandsDetail.NumUATCommands > 0)
		{
			OutUATCommandsDetail.MaxOrder = INT32_MIN;
			OutUATCommandsDetail.MinOrder = INT32_MAX;

			for (const ILauncherProfileUATCommandRef& UATCommand : UATCommands)
			{
				int32 Order = UATCommand->GetOrder();

				OutUATCommandsDetail.MaxOrder = FMath::Max(OutUATCommandsDetail.MaxOrder, Order);
				OutUATCommandsDetail.MinOrder = FMath::Min(OutUATCommandsDetail.MinOrder, Order);
			}
		}
	}







	TArray<EContentScheme> GetAllContentSchemes()
	{
		TArray<EContentScheme> ContentSchemes;
		for (uint8 Index = 0; Index < (uint8)EContentScheme::MAX; Index++)
		{
			ContentSchemes.Add((EContentScheme)Index);
		}

		return MoveTemp(ContentSchemes);
	}

	FText GetContentSchemeDisplayName(EContentScheme ContentScheme)
	{
		switch (ContentScheme)
		{
			case EContentScheme::PakFiles:           return LOCTEXT("ContentSchemePakFiles","Pak Files"); 
			case EContentScheme::ZenStreaming:       return LOCTEXT("ContentSchemeZenStreaming","Zen Streaming");
			case EContentScheme::ZenPakStreaming:    return LOCTEXT("ContentSchemeZenPakStreaming","Zen Pak Streaming"); 
			case EContentScheme::DevelopmentPackage: return LOCTEXT("ContentSchemeDevPackage","Development Package");
			case EContentScheme::SubmissionPackage:  return LOCTEXT("ContentSchemeSubPackage","Submission Package");
			case EContentScheme::LooseFiles:         return LOCTEXT("ContentSchemeLooseFiles","Loose Files (legacy)");
			case EContentScheme::CookOnTheFly:       return LOCTEXT("ContentSchemeCOTF","Cook On The Fly"); 
			case EContentScheme::PreStagedBuild:     return LOCTEXT("ContentSchemePreStaged", "Use Current Staged Build");
		}

		checkNoEntry();
		return FText::GetEmpty();

	}

	FText GetContentSchemeToolTip(EContentScheme ContentScheme)
	{
		switch (ContentScheme)
		{
			case EContentScheme::PakFiles:           return LOCTEXT("ContentSchemeTipPakFiles","Store cooked game content in one or more large Pak Files");
			case EContentScheme::ZenStreaming:       return LOCTEXT("ContentSchemeTipZenStreaming","Stream cooked game content from Zen Server");
			case EContentScheme::ZenPakStreaming:    return LOCTEXT("ContentSchemeTipZenPakStreaming","Stream an existing Pak Files build via a Zen");
			case EContentScheme::DevelopmentPackage: return LOCTEXT("ContentSchemeTipDevPackage","Package cooked game content into a single installable package file for development purposes, where available");
			case EContentScheme::SubmissionPackage:  return LOCTEXT("ContentSchemeTipSubPackage","Package cooked game content into a single package or patch for submission to a game store, where available");
			case EContentScheme::LooseFiles:         return LOCTEXT("ContentSchemeTipLooseFiles","Store cooked game assets in individual files (legacy - recommend moving to Zen Streaming. This option will not work if the project is already configured with 'Use Zen Store')");
			case EContentScheme::CookOnTheFly:       return LOCTEXT("ContentSchemeTipCOTF","Only cook game assets when the game requires them, and send them over the network (legacy - slow)");
			case EContentScheme::PreStagedBuild:     return LOCTEXT("ContentSchemeTipPreStaged","Use the existing staged build as-is. Works best with the Build Sync extension or Build Storage Tool etc");
		}

		checkNoEntry();
		return FText::GetEmpty();

	}




	const TCHAR* LexToString( const ProjectLauncher::EContentScheme& ContentScheme)
	{
		switch (ContentScheme)
		{
			case ProjectLauncher::EContentScheme::PakFiles:           return TEXT("PakFiles");
			case ProjectLauncher::EContentScheme::ZenStreaming:       return TEXT("ZenStreaming");
			case ProjectLauncher::EContentScheme::ZenPakStreaming:    return TEXT("ZenPakStreaming");
			case ProjectLauncher::EContentScheme::DevelopmentPackage: return TEXT("DevelopmentPackage");
			case ProjectLauncher::EContentScheme::SubmissionPackage:  return TEXT("SubmissionPackage");
			case ProjectLauncher::EContentScheme::LooseFiles:         return TEXT("LooseFiles");
			case ProjectLauncher::EContentScheme::CookOnTheFly:       return TEXT("CookOnTheFly");
			case ProjectLauncher::EContentScheme::PreStagedBuild:     return TEXT("PreStagedBuild");
			default: return TEXT("Unknown");
		}
	}



	bool LexTryParseString( ProjectLauncher::EContentScheme& OutContentScheme, const TCHAR* String )
	{
		if (FCString::Stricmp(String, TEXT("PakFiles")) == 0) 
		{
			OutContentScheme = ProjectLauncher::EContentScheme::PakFiles;
			return true;
		}
		else if (FCString::Stricmp(String, TEXT("ZenStreaming")) == 0) 
		{
			OutContentScheme = ProjectLauncher::EContentScheme::ZenStreaming;
			return true;
		}
		else if (FCString::Stricmp(String, TEXT("ZenPakStreaming")) == 0) 
		{
			OutContentScheme = ProjectLauncher::EContentScheme::ZenPakStreaming;
			return true;
		}
		else if (FCString::Stricmp(String, TEXT("DevelopmentPackage")) == 0) 
		{
			OutContentScheme = ProjectLauncher::EContentScheme::DevelopmentPackage;
			return true;
		}
		else if (FCString::Stricmp(String, TEXT("SubmissionPackage")) == 0)
		{
			OutContentScheme = ProjectLauncher::EContentScheme::SubmissionPackage;
			return true;
		}
		else if (FCString::Stricmp(String, TEXT("LooseFiles")) == 0) 
		{
			OutContentScheme = ProjectLauncher::EContentScheme::LooseFiles;
			return true;
		}
		else if (FCString::Stricmp(String, TEXT("CookOnTheFly")) == 0) 
		{
			OutContentScheme = ProjectLauncher::EContentScheme::CookOnTheFly;
			return true;
		}
		else if (FCString::Stricmp(String, TEXT("PreStagedBuild")) == 0) 
		{
			OutContentScheme = ProjectLauncher::EContentScheme::PreStagedBuild;
			return true;
		}
		else
		{
			return false;
		}
	}




	static void GetUATCommandErrorMessage(FTextBuilder& MsgTextBuilder, ILauncherProfilePtr Profile, const ILauncherProfileUATCommandPtr& UATCommand, bool bWithAnnotations, bool bWithUATCommand, bool& bOutHasErrors, bool& bOutHasWarnings)
	{
		TArray<FString> CustomErrors = Profile->GetAllCustomErrors(UATCommand);
		TArray<FString> CustomWarnings = Profile->GetAllCustomWarnings(UATCommand);

		bool bShowUATCommand = (bWithUATCommand && UATCommand.IsValid() && (CustomErrors.Num() > 0 || CustomWarnings.Num() > 0 || Profile->HasValidationError(UATCommand)));
		if (bShowUATCommand)
		{
			MsgTextBuilder.AppendLine(FText::FromString(UATCommand->GetDescription()));
		}

		if (CustomErrors.Num() > 0 || Profile->HasValidationError(UATCommand))
		{
			bOutHasErrors = true;
			for (int i = 0; i < (int)ELauncherProfileValidationErrors::Count; i++)
			{
				ELauncherProfileValidationErrors::Type Error = (ELauncherProfileValidationErrors::Type)i;
				if (Profile->HasValidationError(Error, UATCommand))
				{
					MsgTextBuilder.AppendLineFormat(LOCTEXT("ValidationErrFmt", "Error: {0}"), FText::FromString(LexToStringLocalized(Error)));
				}
			}
			for (const FString& CustomError : CustomErrors)
			{
				MsgTextBuilder.AppendLineFormat(LOCTEXT("CustomErrFmt", "Error: {0}"), Profile->GetCustomErrorText(CustomError, UATCommand));
			}
		}

		if (CustomWarnings.Num() > 0)
		{
			bOutHasWarnings = true;
			for (const FString& CustomWarning : CustomWarnings)
			{
				MsgTextBuilder.AppendLineFormat(LOCTEXT("CustomWarnFmt", "Warning: {0}"), Profile->GetCustomWarningText(CustomWarning, UATCommand));
			}
		}

		if (bShowUATCommand)
		{
			MsgTextBuilder.Unindent();
			MsgTextBuilder.AppendLine();
		}
	}

	FText GetProfileLaunchErrorMessage(ILauncherProfilePtr Profile, bool bWithAnnotations)
	{
		if (!Profile.IsValid())
		{
			return LOCTEXT("LaunchErrNoProfileTip", "There is no profile selected");
		}

		bool bHasErrors = false;
		bool bHasWarnings = false;
		FTextBuilder MsgTextBuilder;

		if (Profile->GetFirstBuildCookRun() == nullptr)
		{
			// global errors are merged into the first build cook run for legacy PL, so we only need to list them separately if we don't have a build cook run
			GetUATCommandErrorMessage(MsgTextBuilder, Profile, nullptr, bWithAnnotations, false, bHasErrors, bHasWarnings);
		}

		bool bWithUATCommand = (Profile->GetUATCommands().Num() > 1);
		for (const ILauncherProfileUATCommandRef& UATCommand : Profile->GetUATCommands())
		{
			GetUATCommandErrorMessage(MsgTextBuilder, Profile, UATCommand.ToSharedPtr(), bWithAnnotations, bWithUATCommand, bHasErrors, bHasWarnings);
		}

		if (bWithAnnotations)
		{
			if (bHasErrors)
			{
				MsgTextBuilder.AppendLine(LOCTEXT("LaunchErrValidation", "There are validation errors with this profile. Please fix them before launching."));
			}
			if (bHasWarnings)
			{
				MsgTextBuilder.AppendLine(LOCTEXT("LaunchWarnValidation", "There are validation warnings with this profile but these will not prevent launching."));
			}
			if (!bHasErrors)
			{
				MsgTextBuilder.AppendLine(LOCTEXT("LaunchProfileTip", "Launch this profile now"));
			}
		}


		return MsgTextBuilder.ToText();
	}



	bool GetUserConfirmation( const FText& Message, const FText& Title, bool bDefaultValue )
	{
		TSharedRef<SMessageDialog> Dialog = SNew(SMessageDialog)
			.Title(Title)
			.Message(Message)
			.Buttons({SMessageDialog::FButton(LOCTEXT("YES","Yes")), SMessageDialog::FButton(LOCTEXT("NO","No"))})
		;

		int32 DefaultButtonIndex = bDefaultValue ? 0 : 1;
		if (Dialog->ShowModal(DefaultButtonIndex) != 0)
		{
			return false;
		}

		return true;
	}


}



#undef LOCTEXT_NAMESPACE
