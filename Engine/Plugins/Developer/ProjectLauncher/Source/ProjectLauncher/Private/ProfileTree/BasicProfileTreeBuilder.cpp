// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileTree/BasicProfileTreeBuilder.h"

#define LOCTEXT_NAMESPACE "BasicProfileTreeBuilder"

namespace ProjectLauncher
{
	FBasicProfileTreeBuilder::FBasicProfileTreeBuilder( const ILauncherProfileRef& InProfile, const TSharedRef<FModel>& InModel )
		: FGenericProfileTreeBuilder( InProfile, InModel->GetDefaultBasicLaunchProfile(), InModel )
	{
		DefaultBuildCookRun = Model->GetBasicDefaultBuildCookRun();
	}


	void FBasicProfileTreeBuilder::Construct()
	{
		FGenericProfileTreeBuilder::Construct();

		ILauncherProfileBuildCookRunRef UATCommand = Profile->GetFirstBuildCookRun().ToSharedRef();
		FBuildCookRun& BuildCookRun = Get(UATCommand);

		FLaunchProfileTreeNode& GeneralSettingsHeader = TreeData->AddHeading( FLaunchProfileTreeData::GeneralSettingsSectionName, LOCTEXT("GeneralSettingsHeading", "General Settings"), FLaunchProfileTreeData::GeneralSettingsSortOrder );
		GeneralSettingsHeader.UATCommand = UATCommand;

		AddProjectProperty(GeneralSettingsHeader);
		BuildCookRun.AddTargetProperty(GeneralSettingsHeader);
		BuildCookRun.AddConfigurationProperty(GeneralSettingsHeader);
		BuildCookRun.AddContentSchemeProperty(GeneralSettingsHeader);
		BuildCookRun.AddTargetDeviceProperty(GeneralSettingsHeader);
		BuildCookRun.AddCommandLineProperty(GeneralSettingsHeader);
	}



	TSharedPtr<ILaunchProfileTreeBuilder> FBasicProfileTreeBuilderFactory::TryCreateTreeBuilder( const ILauncherProfileRef& InProfile, const TSharedRef<FModel>& InModel )
	{
		return MakeShared<FBasicProfileTreeBuilder>(InProfile, InModel);
	}


}

#undef LOCTEXT_NAMESPACE
