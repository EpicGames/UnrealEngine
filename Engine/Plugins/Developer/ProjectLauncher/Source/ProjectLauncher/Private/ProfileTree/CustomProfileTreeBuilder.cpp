// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileTree/CustomProfileTreeBuilder.h"
#include "Model/ProjectLauncherModel.h"
#include "Extension/LaunchExtension.h"
#include "Extension/BuildCookRunCommandExtension.h"


#define LOCTEXT_NAMESPACE "CustomProfileTreeBuilder"

namespace ProjectLauncher
{
	FCustomProfileTreeBuilder::FCustomProfileTreeBuilder( const ILauncherProfileRef& InProfile, const TSharedRef<FModel>& InModel )
		: FGenericProfileTreeBuilder( InProfile, InModel->GetDefaultCustomLaunchProfile(), InModel )
	{
	}

	void FCustomProfileTreeBuilder::Construct()
	{
		FGenericProfileTreeBuilder::Construct();

		// when there's only one profile and no active BuildCookRun extensions, group the target, platform etc. under the general settings header
		// @todo: explore whether this can be handled via SCustomLaunchCustomProfileEditor checking bClassicView instead
		bool bClassicView = Model->CanUseSimplifiedLayout(Profile);
		auto AddHeading = [this, bClassicView]( FLaunchProfileTreeNode& GeneralBuildCookRunHeader, const TCHAR* Name, const FText& Heading ) -> FLaunchProfileTreeNode&
		{
			if (bClassicView)
			{
				FLaunchProfileTreeNode& Header = TreeData->AddHeading( Name, Heading );
				Header.UATCommand = Profile->GetFirstBuildCookRun();
				return Header;
			}
			else
			{
				return GeneralBuildCookRunHeader.AddSubHeading( Name, Heading );
			}
		};

		FLaunchProfileTreeNode& GeneralSettingsHeader = TreeData->AddHeading( FLaunchProfileTreeData::GeneralSettingsSectionName, LOCTEXT("GeneralSettingsHeading", "General Settings"), FLaunchProfileTreeData::GeneralSettingsSortOrder );
		if (bClassicView)
		{
			GeneralSettingsHeader.UATCommand = Profile->GetFirstBuildCookRun();
		}

		AddProjectProperty(GeneralSettingsHeader);
		for (ILauncherProfileBuildCookRunRef UATCommand : Profile->GetBuildCookRunCommands())
		{
			FBuildCookRun& BuildCookRun = Get(UATCommand);

			const bool bIsPrimaryHeading = true;
			FLaunchProfileTreeNode& GeneralBuildCookRunHeader = bClassicView ? GeneralSettingsHeader : TreeData->FindOrAddHeading(UATCommand, bIsPrimaryHeading);

			BuildCookRun.AddTargetProperty(GeneralBuildCookRunHeader);
			BuildCookRun.AddPlatformProperty(GeneralBuildCookRunHeader);
			BuildCookRun.AddArchitectureProperty(GeneralBuildCookRunHeader);
			BuildCookRun.AddConfigurationProperty(GeneralBuildCookRunHeader);
			BuildCookRun.AddContentSchemeProperty(GeneralBuildCookRunHeader);

			FLaunchProfileTreeNode& ContentSchemeHeader = AddHeading( GeneralBuildCookRunHeader, TEXT("ContentScheme"), LOCTEXT("ContentSchemeHeading", "Content Scheme") );
			BuildCookRun.AddCompressPakFilesProperty(ContentSchemeHeader);
			BuildCookRun.AddGenerateChunksProperty(ContentSchemeHeader);
			BuildCookRun.AddZenPakStreamingPathProperty(ContentSchemeHeader);

			FLaunchProfileTreeNode& SubmissionHeader = AddHeading( GeneralBuildCookRunHeader,  TEXT("Release"), LOCTEXT("ReleaseHeading", "Release / DLC / Patching") );
			BuildCookRun.AddSubmissionPackageProperty(SubmissionHeader);

			FLaunchProfileTreeNode& CookingHeader = AddHeading( GeneralBuildCookRunHeader,  TEXT("Cooking"), LOCTEXT("CookingHeading", "Cook") );
			BuildCookRun.AddCookProperty(CookingHeader);
			BuildCookRun.AddIncrementalCookProperty(CookingHeader);

			FLaunchProfileTreeNode& AdvCookingHeader = AddHeading( GeneralBuildCookRunHeader,  TEXT("AdvancedCook"), LOCTEXT("AdvancedCookingHeading", "Advanced Cooking Options") ); // @fixme: hack - just so Advanced Cooking Extension panel appears directly below normal cook. @todo: Need 'InsertAfter' for headings
			FLaunchProfileTreeNode& BuildHeader = AddHeading( GeneralBuildCookRunHeader,  TEXT("Build"), LOCTEXT("BuildHeading", "Build") );
			BuildCookRun.AddBuildProperty(BuildHeader);
			BuildCookRun.AddForceBuildProperty(BuildHeader);

			FLaunchProfileTreeNode& DirectoryHeader = AddHeading( GeneralBuildCookRunHeader,  TEXT("Directory"), LOCTEXT("DirectoryHeading", "Directory") );
			BuildCookRun.AddArchiveBuildProperty(DirectoryHeader);
			BuildCookRun.AddArchiveBuildDirectoryProperty(DirectoryHeader);
			BuildCookRun.AddStagingDirectoryProperty(DirectoryHeader);

			FLaunchProfileTreeNode& DeployAndRunHeader = AddHeading( GeneralBuildCookRunHeader,  TEXT("DeployAndRun"), LOCTEXT("DeployAndRunHeading", "Deploy And Run") );
			BuildCookRun.AddDeployProperty(DeployAndRunHeader);
			BuildCookRun.AddTargetDeviceProperty(DeployAndRunHeader);
			BuildCookRun.AddRunProperty(DeployAndRunHeader);
			BuildCookRun.AddInitialMapProperty(DeployAndRunHeader);
			BuildCookRun.AddCommandLineProperty(DeployAndRunHeader);
		}
	}



	TSharedPtr<ILaunchProfileTreeBuilder> FCustomProfileTreeBuilderFactory::TryCreateTreeBuilder( const ILauncherProfileRef& InProfile, const TSharedRef<FModel>& InModel )
	{
		return MakeShared<FCustomProfileTreeBuilder>(InProfile, InModel);
	}


}

#undef LOCTEXT_NAMESPACE
