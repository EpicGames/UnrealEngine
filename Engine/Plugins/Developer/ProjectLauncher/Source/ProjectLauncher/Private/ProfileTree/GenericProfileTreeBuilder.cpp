// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileTree/GenericProfileTreeBuilder.h"
#include "Widgets/Shared/SCustomLaunchProjectCombo.h"
#include "Widgets/Shared/SCustomLaunchBuildTargetCombo.h"
#include "Widgets/Shared/SCustomLaunchPlatformCombo.h"
#include "Widgets/Shared/SCustomLaunchContentSchemeCombo.h"
#include "Widgets/Shared/SCustomLaunchDeviceListView.h"
#include "Widgets/Shared/SCustomLaunchCombo.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "SResizeBox.h"
#include "PlatformInfo.h"
#include "IDesktopPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "GameProjectHelper.h"
#include "InstalledPlatformInfo.h"
#include "HAL/PlatformMisc.h"


#define LOCTEXT_NAMESPACE "CustomProfileTreeBuilder"


namespace ProjectLauncher
{
	FGenericProfileTreeBuilder::FGenericProfileTreeBuilder( const ILauncherProfileRef& InProfile, const ILauncherProfileRef& InDefaultProfile, const TSharedRef<FModel>& InModel )
		: TreeData( MakeShared<FLaunchProfileTreeData>(InProfile, InModel, this) )
		, Profile(InProfile)
		, DefaultProfile(InDefaultProfile)
		, Model(InModel)
	{
		ProfileType = Model->GetProfileType(Profile);
		DefaultBuildCookRun = Model->GetCustomDefaultBuildCookRun();

		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : InProfile->GetBuildCookRunCommands())
		{
			BuildCookRunData.Add(BuildCookRun, MakeShared<FBuildCookRun>(*this, BuildCookRun, Model));
		}

		Profile->OnUATCommandAdded().AddRaw( this, &FGenericProfileTreeBuilder::OnUATCommandAdded );
		Profile->OnUATCommandRemoved().AddRaw( this, &FGenericProfileTreeBuilder::OnUATCommandRemoved );

		Model->OnProjectSettingsReady().AddRaw(this, &FGenericProfileTreeBuilder::ValidateBuildTargets );

	}

	FGenericProfileTreeBuilder::~FGenericProfileTreeBuilder()
	{
		Model->OnProjectSettingsReady().RemoveAll(this);
		Profile->OnUATCommandAdded().RemoveAll(this);
		Profile->OnUATCommandRemoved().RemoveAll(this);
	}



	FGenericProfileTreeBuilder::FBuildCookRun::FBuildCookRun( FGenericProfileTreeBuilder& InOwner, ILauncherProfileBuildCookRunRef InBuildCookRun, const TSharedRef<FModel>& InModel )
		: Owner(InOwner)
		, BuildCookRun(InBuildCookRun)
		, Model(InModel)
	{

		ForPak = [this]()
		{
			return (ContentScheme == EContentScheme::PakFiles || ContentScheme == EContentScheme::DevelopmentPackage || ContentScheme == EContentScheme::SubmissionPackage);
		};

		ForZen = [this]()
		{
			return (ContentScheme == EContentScheme::ZenStreaming || ContentScheme == EContentScheme::ZenPakStreaming);
		};

		ForZenWS = [this]()
		{
			return (ContentScheme == EContentScheme::ZenPakStreaming);
		};

		ForCooked = [this]()
		{
			return (ContentScheme != EContentScheme::ZenPakStreaming && ContentScheme != EContentScheme::CookOnTheFly && ContentScheme != EContentScheme::PreStagedBuild);
		};

		ForEnabledCooked = [this]()
		{
			return (bShouldCook && ContentScheme != EContentScheme::ZenPakStreaming && ContentScheme != EContentScheme::CookOnTheFly && ContentScheme != EContentScheme::PreStagedBuild);
		};

		ForContent = [this]()
		{
			return (ContentScheme != EContentScheme::ZenPakStreaming && ContentScheme != EContentScheme::PreStagedBuild);
		};

		ForCode = [this]()
		{
			return GetBuild();
		};

		ForCodeBuild = [this]()
		{
			return (ContentScheme != EContentScheme::PreStagedBuild);
		};

		ForDeployment = [this]()
		{
			return GetDeployToDevice();
		};

		ForRun = [this]()
		{
			return GetIsRunning();
		};

		ForSubmissionPackage = [this]()
		{
			return (ContentScheme == EContentScheme::SubmissionPackage);
		};

		EmptyString = []()
		{
			return FString();
		};

		Owner.Profile->OnCustomUATCommandValidation().AddRaw( this, &FBuildCookRun::OnValidateProfile );
		Model->OnProjectSettingsReady().AddRaw( this, &FBuildCookRun::OnProjectSettingsReady );
	}
	
	FGenericProfileTreeBuilder::FBuildCookRun::~FBuildCookRun()
	{
		Owner.Profile->OnCustomUATCommandValidation().RemoveAll( this );
		Model->OnProjectSettingsReady().RemoveAll( this );
	}


	void FGenericProfileTreeBuilder::Construct()
	{
		for (const TPair<ILauncherProfileBuildCookRunRef,TSharedRef<FBuildCookRun>>& Pair : BuildCookRunData)
		{
			Pair.Value->Construct();
		}
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::Construct()
	{
		const TArray<FString>& DeviceIDs = BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs();

		bShouldCook = GetCook();
		bHasAdvancedPlatformTargets = (BuildCookRun->GetCookedPlatforms().Num() > 1 || BuildCookRun->GetBuildTargets().Num() > 1);
		ContentScheme = Model->DetermineProfileContentScheme(BuildCookRun);

		// ensure the default deploy platform matches the cook platform because Legacy Project Launcher can create a mismatch
		if (BuildCookRun->GetDefaultDeployPlatform() != NAME_None && !GetSelectedPlatforms().Contains(BuildCookRun->GetDefaultDeployPlatform().ToString()) )
		{
			FString DeployPlatform = (GetSelectedPlatforms().Num() > 0) ? GetSelectedPlatforms()[0] : FPlatformProperties::IniPlatformName();
			BuildCookRun->SetDefaultDeployPlatform(FName(*DeployPlatform));
		}

		if (InitalMapCombo.IsValid())
		{
			InitalMapCombo->SetSelectedItem( GetInitialMap(), ESelectInfo::Direct );
			InitalMapCombo->RefreshOptions();
		}

		CacheArchitectures();
		CacheBuildTargets();

	}






	void FGenericProfileTreeBuilder::AddProjectProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		if (ProfileType == EProfileType::Basic)
		{
			// this is adding a new widget to the property tree
			// - the first entry is the name of the property on the left-hand side
			// - the seecond parameter is the widget itself that appears on the right-hand side
			HeadingNode.AddWidget( LOCTEXT("ProjectLabel", "Project"), 
				{
					.Validation = FValidation( {ELauncherProfileValidationErrors::NoProjectSelected} ),
				},
				SNew(SCustomLaunchProjectCombo)
				.OnSelectionChanged(this, &FGenericProfileTreeBuilder::SetProjectName)
				.SelectedProject(this, &FGenericProfileTreeBuilder::GetProjectPath)
				.HasProject(this, &FGenericProfileTreeBuilder::HasProject)
				.CurrentProjectOption(SCustomLaunchProjectCombo::ECurrentProjectOption::Empty)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			);
		}
		else
		{
			HeadingNode.AddWidget( LOCTEXT("ProjectLabel", "Project"), 
				{
					.Validation = FValidation( {ELauncherProfileValidationErrors::NoProjectSelected} ),
				},
				SNew(SCustomLaunchProjectCombo)
				.OnSelectionChanged(this, &FGenericProfileTreeBuilder::SetProjectName)
				.SelectedProject(this, &FGenericProfileTreeBuilder::GetProjectPath)
				.HasProject(this, &FGenericProfileTreeBuilder::HasProject)
				.ShowAnyProjectOption(true)
				.CurrentProjectOption(SCustomLaunchProjectCombo::ECurrentProjectOption::ActualProject)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			);
		}
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddTargetProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		if (Owner.ProfileType == EProfileType::Basic)
		{
			// this is also adding a new widget to the property tree, as above.
			// in this example the new struct parameter defines several callbacks that handle the 'reset to default' functionality. there are also options for disabling & hiding.
			// the code is implemented with this syntax to avoid aid readability without using slate's TAttribute style functionality which seemed like an overkill for our simpler needs.
			HeadingNode.AddWidget( LOCTEXT("TargetLabel", "Target"), 
				{
					.IsDefault = [this]()		{ return !BuildCookRun->HasBuildTargetSpecified() || (BuildCookRun->GetBuildTargets().Num() == 0); },
					.SetToDefault = [this]()	{ SetBuildTargets(TArray<FString>()); },
					.IsEnabled = [this]()		{ return Model->AreProjectSettingsReady(Owner.Profile); },
					.Validation = FValidation( {ELauncherProfileValidationErrors::BuildTargetCookVariantMismatch, ELauncherProfileValidationErrors::BuildTargetIsRequired, ELauncherProfileValidationErrors::CookOnTheFlyDoesntSupportServer} ),
				},
				SAssignNew(BuildTargetCombo, SCustomLaunchBuildTargetCombo, Model)
				.OnSelectionChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetBuildTargets)
				.SelectedBuildTargets(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetBuildTargets)
				.SelectedProject(Owner.AsShared(), &FGenericProfileTreeBuilder::GetProjectPath)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			);
		}
		else
		{
			HeadingNode.AddWidget( LOCTEXT("TargetLabel", "Target"), 
				{
					.IsDefault = [this]()		{ return !BuildCookRun->HasBuildTargetSpecified() || (BuildCookRun->GetBuildTargets().Num() == 0); },
					.SetToDefault = [this]()	{ SetBuildTargets(TArray<FString>()); },
					.IsEnabled = [this]()		{ return Owner.Profile->HasProjectSpecified() && Model->AreProjectSettingsReady(Owner.Profile); },
					.Validation = FValidation( {ELauncherProfileValidationErrors::BuildTargetCookVariantMismatch, ELauncherProfileValidationErrors::BuildTargetIsRequired, ELauncherProfileValidationErrors::CookOnTheFlyDoesntSupportServer}, {TEXT("Validation_GameAndClientTargetsDisallowed")}),
				},
				SAssignNew(BuildTargetCombo, SCustomLaunchBuildTargetCombo, Model)
				.OnSelectionChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetBuildTargets)
				.SelectedBuildTargets(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetBuildTargets)
				.SelectedProject(Owner.AsShared(), &FGenericProfileTreeBuilder::GetProjectPath)
				//.SetAdvancedPlatformsOption(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetHasAdvancedPlatformTargets) // @todo: prevent this from being toggled until the final design on client/server build configuration. only exising multi-target/multi-platform profiles will have multiselectable platforms/target controls
				.GetAdvancedPlatformsOption(this, &FGenericProfileTreeBuilder::FBuildCookRun::HasAdvancedPlatformTargets)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			);
		}
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddPlatformProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddWidget( LOCTEXT("PlatformLabel", "Platform"), 
			{
				.IsEnabled = [this]() { return Model->AreProjectSettingsReady(Owner.Profile); },
				.Validation = FValidation({ELauncherProfileValidationErrors::NoPlatformSelected, ELauncherProfileValidationErrors::NoPlatformSDKInstalled}),
			},
			SAssignNew(PlatformCombo, SCustomLaunchPlatformCombo)
			.SelectedPlatforms(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetSelectedPlatforms)
			.OnSelectionChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetSelectedPlatforms)
			.GetAdvancedPlatformsOption(this, &FGenericProfileTreeBuilder::FBuildCookRun::HasAdvancedPlatformTargets)
			.SupportedBuildTargets_Lambda( [this]() { return CachedBuildTargetTypes; } )
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
		);
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::AddConfigurationProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		TArray<EBuildConfiguration> ValidConfigurations;
		
		static const EBuildConfiguration AllConfigurations[] = { EBuildConfiguration::Debug, EBuildConfiguration::DebugGame, EBuildConfiguration::Development, EBuildConfiguration::Test, EBuildConfiguration::Shipping };
		for (EBuildConfiguration Configuration : AllConfigurations)
		{
			// only show the configurations that are currently available. @todo: might be better to show all, but disable the ones that are unavailale
			if (FInstalledPlatformInfo::Get().IsValidConfiguration(Configuration))
			{
				ValidConfigurations.Add(Configuration);
			}
		}

		HeadingNode.AddWidget( LOCTEXT("ConfigurationLabel", "Configuration"), 
			{
				.Validation = FValidation({ELauncherProfileValidationErrors::NoBuildConfigurationSelected, ELauncherProfileValidationErrors::ShippingDoesntSupportCommandlineOptionsCantUseCookOnTheFly}),
			},
			SNew(SCustomLaunchLexToStringCombo<EBuildConfiguration>)
			.OnSelectionChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetBuildConfiguration)
			.SelectedItem(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetBuildConfiguration)
			.Items(ValidConfigurations)
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddContentSchemeProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddWidget( LOCTEXT("ContentSchemeLabel", "Content Scheme"), 
			{
				.Validation = FValidation( {ELauncherProfileValidationErrors::CookOnTheFlyDoesntSupportServer, ELauncherProfileValidationErrors::ShippingDoesntSupportCommandlineOptionsCantUseCookOnTheFly, ELauncherProfileValidationErrors::ZenPakStreamingRequiresDeployAndLaunch}, 
											{TEXT("Validation_ZenNeedsIoStore"), TEXT("Validation_ZenNeedsRemotePermission")} ),
			},
			SNew(SCustomLaunchContentSchemeCombo)
			.OnSelectionChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetContentScheme)
			.SelectedContentScheme_Lambda( [this]() { return ContentScheme; } )
			.IsContentSchemeAvailable(this, &FGenericProfileTreeBuilder::FBuildCookRun::IsContentSchemeAvailable)
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddCompressPakFilesProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		// in this example we are adding a single boolean instead of a custom widget. the struct parameter defines how the value is accessed
		// the ForPak is the lambda functions created in the constructor and is again aimed at readability
		HeadingNode.AddBoolean( LOCTEXT("CompressPakFilesLabel", "Compress Pak Files"),
			{
				.GetValue = [this]()			{ return BuildCookRun->IsCompressed(); },
				.SetValue = [this](bool bValue)	{ BuildCookRun->SetCompressed(bValue); },
				.GetDefaultValue = [this]()		{ return Owner.DefaultBuildCookRun->IsCompressed(); },
				.IsVisible = ForPak,
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddGenerateChunksProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("GenerateChunksLabel", "Generate Chunks"),
			{
				.GetValue = [this]()			{ return BuildCookRun->IsGeneratingChunks(); },
				.SetValue = [this](bool bValue)	{ BuildCookRun->SetGenerateChunks(bValue); },
				.GetDefaultValue = [this]()		{ return Owner.DefaultBuildCookRun->IsGeneratingChunks(); },
				.IsVisible = ForPak,
				.Validation = FValidation({ELauncherProfileValidationErrors::GeneratingChunksRequiresCookByTheBook, ELauncherProfileValidationErrors::GeneratingChunksRequiresUnrealPak}),
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddZenSnapshotProperty(FLaunchProfileTreeNode& HeadingNode) // @todo: could revisit how this is displayed as it's global? could be inline next to AddImportZenSnapshotProperty tickbox?
	{
		HeadingNode.AddWidget(LOCTEXT("ZenSnapshotLabel", "Closest Zen Snapshot"),
			{
				.IsVisible = [this]() { return ForContent(); }
			},
			SNew(STextBlock)
			.Text(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetZenSnapshotText)
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont")));
	}

	FText FGenericProfileTreeBuilder::FBuildCookRun::GetZenSnapshotText() const
	{
		int32 Build = Owner.Profile->GetZenSnapshot();
		return FText::FromString(FString::FromInt(Build));
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddImportZenSnapshotProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("ImportZenSnapshotLabel", "Import Best Match Zen Snapshot"),
			{
				.GetValue = [this]()			{ return BuildCookRun->IsImportingZenSnapshot(); },
				.SetValue = [this](bool bValue)	{ BuildCookRun->SetImportingZenSnapshot(bValue); },
				.GetDefaultValue = [this]()		{ return Owner.DefaultBuildCookRun->IsImportingZenSnapshot(); },
				.IsVisible = ForContent,
				.Validation = FValidation({ELauncherProfileValidationErrors::CopyToDeviceRequiresCookByTheBook}),
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddZenPakStreamingPathProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddDirectoryString( LOCTEXT("ZenPakStreamingPathLabel", "Zen Pak Streaming Path"),
			{
				.GetValue = [this]()				{ return BuildCookRun->GetZenPakStreamingPath(); },
				.SetValue = [this](FString Value)	{ BuildCookRun->SetZenPakStreamingPath(Value); },
				.GetDefaultValue = EmptyString,
				.IsVisible = ForZenWS,
				.Validation = FValidation({ELauncherProfileValidationErrors::ZenPakStreamingRequiresDeployAndLaunch}),
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddSubmissionPackageProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		auto TryGetTargetPlatform = [this](const ITargetPlatform*& OutPlatform)
		{
			const TArray<FString>& CookedPlatforms = BuildCookRun->GetCookedPlatforms();
			if (CookedPlatforms.Num() == 1)
			{
				const TArray<ITargetPlatform*>& TargetPlatforms = GetTargetPlatformManager()->GetTargetPlatforms();
				for (const ITargetPlatform* TargetPlatform : TargetPlatforms)
				{
					if (CookedPlatforms[0] == TargetPlatform->PlatformName())
					{
						OutPlatform = TargetPlatform;
						return true;
					}
				}
			}

			OutPlatform = nullptr;
			return false;
		};

		auto HasNonTrivialPatchGeneration = [TryGetTargetPlatform]()
		{
			const ITargetPlatform* Platform;
			return TryGetTargetPlatform(Platform) && Platform->HasNonTrivialPatchGeneration();
		};

		auto IsBasedOnReleaseVersionVisible = [this]
		{
			return ForSubmissionPackage() && (BuildCookRun->IsCreatingDLC() || BuildCookRun->IsGeneratingPatch());
		};

		auto IsOriginalReleaseVersionVisible = [this, TryGetTargetPlatform]
		{
			const ITargetPlatform* Platform;
			return ForSubmissionPackage() && TryGetTargetPlatform(Platform) && Platform->RequiresOriginalReleaseVersionForPatch() && BuildCookRun->IsGeneratingPatch();
		};


		// ... warning banner ...
		HeadingNode.AddWidget( FText::GetEmpty(),
			{
				.IsVisible = [this,HasNonTrivialPatchGeneration]() { return ForSubmissionPackage() && HasNonTrivialPatchGeneration(); }
			},
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.LightGroupBorder"))
			.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Warning"))
			[
				SNew(STextBlock)
				.Margin(FMargin(8,2))
				.ColorAndOpacity(FLinearColor::White)
				.AutoWrapText(true)
				.Text(LOCTEXT("ComplexPkgInfo", "This platform has some additional complexities for creating submission packages and patches. Please refer to the relevant documentation for details."))
			]
		)

		// ... release version ...
		.AddBoolean( LOCTEXT("CreateReleaseLabel", "Create Release Version"),
			{
				.GetValue = [this]()				{ return BuildCookRun->IsCreatingReleaseVersion(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetCreateReleaseVersion(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->IsCreatingReleaseVersion(); },
				.IsVisible = ForSubmissionPackage,
			}
		)
		.AddString( LOCTEXT("ReleaseVersionNameLabel", "Release Version Name"),
			{
				.GetValue = [this]()				{ return BuildCookRun->GetCreateReleaseVersionName(); },
				.SetValue = [this](FString Value)	{ BuildCookRun->SetCreateReleaseVersionName(Value); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->GetCreateReleaseVersionName(); },
				.IsVisible = ForSubmissionPackage,
				.IsEnabled = [this]()				{ return BuildCookRun->IsCreatingReleaseVersion() || BuildCookRun->IsGeneratingPatch(); },
			}
		)

		// ... various release/distribution options ....
		.AddBoolean( LOCTEXT("EncryptIniLabel", "Encrypt ini files"),
			{
				.GetValue = [this]()				{ return BuildCookRun->IsEncryptingIniFiles(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetEncryptingIniFiles(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->IsEncryptingIniFiles(); },
				.IsVisible = ForSubmissionPackage,
			}
		)
		.AddBoolean( LOCTEXT("ForDistributionLabel", "This build is for distribution to the public"),
			{
				.GetValue = [this]()				{ return BuildCookRun->IsForDistribution(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetForDistribution(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->IsForDistribution(); },
				.IsVisible = ForSubmissionPackage,
			}
		)
		.AddBoolean( LOCTEXT("IncludePrereqLabel", "Include an installer for prerequisites"),
			{
				.GetValue = [this]()				{ return BuildCookRun->IsIncludingPrerequisites(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetIncludePrerequisites(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->IsIncludingPrerequisites(); },
				.IsVisible = ForSubmissionPackage,
			}
		)
		.AddBoolean( LOCTEXT("BinaryConfigLabel", "Make a binary config file for faster runtime settings"),
			{
				.GetValue = [this]()				{ return BuildCookRun->MakeBinaryConfig(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetMakeBinaryConfig(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->MakeBinaryConfig(); },
				.IsVisible = ForSubmissionPackage,
			}
		)

		// ... patch/DLC options ...
		.AddBoolean( LOCTEXT("CreatePatchLabel", "Create Patch"),
			{
				.GetValue = [this]()				{ return BuildCookRun->IsGeneratingPatch(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetGeneratePatch(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->IsGeneratingPatch(); },
				.IsVisible = [this]()				{ return ForSubmissionPackage(); },
				.IsEnabled = [this]()				{ return !BuildCookRun->IsCreatingDLC(); },
			}
		)
		.AddBoolean( LOCTEXT("CreateDLCLabel", "Create DLC"),
			{
				.GetValue = [this]()				{ return BuildCookRun->IsCreatingDLC(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetCreateDLC(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->IsCreatingDLC(); },
				.IsVisible = [this]()				{ return ForSubmissionPackage(); },
				.IsEnabled = [this]()				{ return !BuildCookRun->IsGeneratingPatch(); },
			}
		)

		.AddString( LOCTEXT("BasedOnReleaseVersionNameLabel", "Based On Release Version Name"),
			{
				.GetValue = [this]()				{ return BuildCookRun->GetBasedOnReleaseVersionName(); },
				.SetValue = [this](FString Value)	{ BuildCookRun->SetBasedOnReleaseVersionName(Value); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->GetBasedOnReleaseVersionName(); },
				.IsVisible = IsBasedOnReleaseVersionVisible,
			}
		)
		.AddString( LOCTEXT("OriginalReleaseVersionNameLabel", "Original Release Version Name"),
			{
				.GetValue = [this]()				{ return BuildCookRun->GetOriginalReleaseVersionName(); },
				.SetValue = [this](FString Value)	{ BuildCookRun->SetOriginalReleaseVersionName(Value); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->GetOriginalReleaseVersionName(); },
				.IsVisible = IsOriginalReleaseVersionVisible,
			}
		)

		.AddBoolean( LOCTEXT("AddPatchLevelLabel", "Add Patch Level (not generally needed)"),
			{
				.GetValue = [this]()				{ return BuildCookRun->ShouldAddPatchLevel(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetAddPatchLevel(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->ShouldAddPatchLevel(); },
				.IsVisible = [this]()				{ return ForSubmissionPackage() && BuildCookRun->IsGeneratingPatch(); },
			}
		)

		.AddString( LOCTEXT("DLCNameLabel", "DLC Name"),
			{
				.GetValue = [this]()				{ return BuildCookRun->GetDLCName(); },
				.SetValue = [this](FString Value)	{ BuildCookRun->SetDLCName(Value); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->GetDLCName(); },
				.IsVisible = [this]()				{ return ForSubmissionPackage() && BuildCookRun->IsCreatingDLC(); },
			}
		)
		.AddBoolean( LOCTEXT("DLCIncludeEngineLabel", "DLC Includes Engine Content"),
			{
				.GetValue = [this]()				{ return BuildCookRun->IsDLCIncludingEngineContent(); },
				.SetValue = [this](bool bValue)		{ BuildCookRun->SetDLCIncludeEngineContent(bValue); },
				.GetDefaultValue = [this]()			{ return Owner.DefaultBuildCookRun->IsDLCIncludingEngineContent(); },
				.IsVisible = [this]()				{ return ForSubmissionPackage() && BuildCookRun->IsCreatingDLC(); },
			}
		)

		;
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::AddIncrementalCookProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("IncrementalCookLabel", "Incremental Cook"),
			{
				.GetValue = [this]()			{ return BuildCookRun->IsCookingIncrementally(); },
				.SetValue = [this](bool bValue)	{ BuildCookRun->SetIncrementalCooking(bValue); },
				.GetDefaultValue = [this]()		{ return Owner.DefaultBuildCookRun->IsCookingIncrementally(); },
				.IsVisible = ForEnabledCooked,
				.Validation = FValidation({ELauncherProfileValidationErrors::UnversionedAndIncremental}),
			}
		);
	}



	void FGenericProfileTreeBuilder::FBuildCookRun::AddCookProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("CookLabel", "Cook Content"),
			{
				.GetValue = [this]()			{ return GetCook(); },
				.SetValue = [this](bool bValue)	{ SetCook(bValue); },
				.GetDefaultValue = [this]()		{ return GetCook(Owner.DefaultBuildCookRun); },
				.IsVisible = ForCooked,
				.Validation = FValidation({ELauncherProfileValidationErrors::CopyToDeviceRequiresCookByTheBook, ELauncherProfileValidationErrors::GeneratingPatchesCanOnlyRunFromByTheBookCookMode, ELauncherProfileValidationErrors::GeneratingChunksRequiresCookByTheBook}),
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddBuildProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("BuildLabel", "Build the game"),
			{
				.GetValue = [this]()			{ return GetBuild(); },
				.SetValue = [this](bool bValue)	{ SetBuild(bValue); },
				.GetDefaultValue = [this]()		{ return GetBuild(Owner.DefaultBuildCookRun); },
				.IsVisible = ForCodeBuild,
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddForceBuildProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("ForceBuildLabel", "Build even if a pre-built target exists"),
			{
				.GetValue = [this]()			{ return GetForceBuild(); },
				.SetValue = [this](bool bValue)	{ SetForceBuild(bValue); },
				.GetDefaultValue = [this]()		{ return GetForceBuild(Owner.DefaultBuildCookRun); },
				.IsVisible = ForCodeBuild,
				.IsEnabled = ForCode,
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddArchitectureProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		static const struct
		{
			EBuildTargetType TargetType;
			FText Label;
		}
		PerTargetTypeArchitectures[] =
		{
			{ EBuildTargetType::Game,   LOCTEXT("GameArchitectureLabel",   "Game Architecture")   },
			{ EBuildTargetType::Client, LOCTEXT("ClientArchitectureLabel", "Client Architecture") },
			{ EBuildTargetType::Server, LOCTEXT("ServerArchitectureLabel", "Server Architecture") },
			{ EBuildTargetType::Editor, LOCTEXT("EditorArchitectureLabel", "Edtor Architecture") },
		};

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(PerTargetTypeArchitectures); Index++)
		{
			EBuildTargetType TargetType = PerTargetTypeArchitectures[Index].TargetType;

			HeadingNode.AddWidget( PerTargetTypeArchitectures[Index].Label,
				{
					.IsDefault = [this, TargetType]()		{ return GetArchitecture(TargetType).IsEmpty(); },
					.SetToDefault = [this, TargetType]()	{ SetArchitecture(FString(), TargetType); },
					.IsVisible = [this, TargetType]()       { return CachedBuildTargetTypes.Contains(TargetType) && CachedBuildTargetArchitectures[TargetType].Num() > 0; }
				},
				SNew(SCustomLaunchStringCombo)
				.OnSelectionChanged( this, &FGenericProfileTreeBuilder::FBuildCookRun::SetArchitecture, TargetType )
				.SelectedItem( this, &FGenericProfileTreeBuilder::FBuildCookRun::GetArchitecture, TargetType)
				.GetDisplayName( this, &FGenericProfileTreeBuilder::FBuildCookRun::GetArchitectureDisplayName )
				.Items_Lambda( [this, TargetType]() { return CachedBuildTargetArchitectures[TargetType];} )
			);

		}
	}




	void FGenericProfileTreeBuilder::FBuildCookRun::AddStagingDirectoryProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddDirectoryString( LOCTEXT("CustomStagingPathLabel", "Stage Directory Override"),
			{
				.GetValue = [this]()				{ return BuildCookRun->GetPackageDirectory(); },
				.SetValue = [this](FString Value)	{ BuildCookRun->SetPackageDirectory(Value); },
				.GetDefaultValue = EmptyString,
				.IsVisible = ForCooked,
				.Validation = FValidation({ELauncherProfileValidationErrors::NoPackageDirectorySpecified}),
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddArchiveBuildProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("ArchiveBuildLabel", "Archive Build"),
			{
				.GetValue = [this]()			{ return BuildCookRun->IsArchiving(); },
				.SetValue = [this](bool bValue)	{ BuildCookRun->SetArchive(bValue); },
				.GetDefaultValue = [this]()		{ return Owner.DefaultBuildCookRun->IsArchiving(); },
				.IsVisible = ForCooked,
			}
		);
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::AddArchiveBuildDirectoryProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddDirectoryString( LOCTEXT("ArchivePathLabel", "Archive Directory"),
			{
				.GetValue = [this]()				{ return BuildCookRun->GetArchiveDirectory(); },
				.SetValue = [this](FString Value)	{ BuildCookRun->SetArchiveDirectory(Value); },
				.GetDefaultValue = EmptyString,
				.IsVisible = ForCooked,				
				.IsEnabled = [this]()				{ return BuildCookRun->IsArchiving(); },
				.Validation = FValidation({ELauncherProfileValidationErrors::NoArchiveDirectorySpecified}),
			}
		);
	}



	void FGenericProfileTreeBuilder::FBuildCookRun::AddDeployProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("DeployLabel", "Deploy To Device"),
			{
				.GetValue = [this]()			{ return GetDeployToDevice(); },
				.SetValue = [this](bool bValue)	{ SetDeployToDevice(bValue); },
				.GetDefaultValue = [this]()		{ return GetDeployToDevice(Owner.DefaultBuildCookRun); },
				.IsVisible = [this]()			{ return (ContentScheme != EContentScheme::CookOnTheFly) && (ContentScheme != EContentScheme::SubmissionPackage); },
				.Validation = FValidation({ELauncherProfileValidationErrors::CopyToDeviceRequiresCookByTheBook, ELauncherProfileValidationErrors::ZenPakStreamingRequiresDeployAndLaunch, ELauncherProfileValidationErrors::DeployedDeviceGroupRequired }),
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddIncrementalDeployProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("DeployModifiedLabel", "Only Deploy Modified Content"),
			{
				.GetValue = [this]()			{ return BuildCookRun->IsDeployingIncrementally(); },
				.SetValue = [this](bool bValue)	{ BuildCookRun->SetIncrementalDeploying(bValue); },
				.GetDefaultValue = [this]()		{ return Owner.DefaultBuildCookRun->IsDeployingIncrementally(); },
				.IsVisible = [this]()			{ return (ContentScheme != EContentScheme::CookOnTheFly) && (ContentScheme != EContentScheme::SubmissionPackage); },
				.IsEnabled = ForDeployment,
			}
		);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::AddTargetDeviceProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddWidget( LOCTEXT("TargetDeviceLabel", "Target Device"), 
			{
				.IsVisible = [this]()			{ return (ContentScheme != EContentScheme::CookOnTheFly) && (ContentScheme != EContentScheme::SubmissionPackage); },
				.IsEnabled = ForDeployment,
				.Validation = FValidation( {ELauncherProfileValidationErrors::BuildTargetCookVariantMismatch, ELauncherProfileValidationErrors::LaunchDeviceIsUnauthorized} ),
			},
			CreateDeployDeviceWidget()
		);
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::AddRunProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddBoolean( LOCTEXT("RunLabel", "Run"),
			{
				.GetValue = [this]()			{ return GetIsRunning(); },
				.SetValue = [this](bool bValue)	{ SetIsRunning(bValue); },
				.GetDefaultValue = [this]()		{ return GetIsRunning(Owner.DefaultBuildCookRun); },
				.IsVisible = [this]()           { return (ContentScheme != EContentScheme::SubmissionPackage) && Owner.Profile->GetAutomatedTests().Num() == 0; }, // run is implied when there are automated tests
				.Validation = FValidation({ELauncherProfileValidationErrors::ZenPakStreamingRequiresDeployAndLaunch, ELauncherProfileValidationErrors::NoLaunchRoleDeviceAssigned}),
			}
		);
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::AddInitialMapProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		// todo: custom map picker 
		HeadingNode.AddWidget( LOCTEXT("InitialMapLabel", "Initial Map"), 
			{
				.IsDefault = [this]()		{ return BuildCookRun->GetDefaultLaunchRole()->GetInitialMap().IsEmpty(); },
				.SetToDefault = [this]()	{ BuildCookRun->GetDefaultLaunchRole()->SetInitialMap(FString()); InitalMapCombo->SetSelectedItem(GetInitialMap()); },
				.IsVisible = [this]()		{ return (ContentScheme != EContentScheme::SubmissionPackage); },
				.IsEnabled = ForRun,
				.Validation = FValidation({ELauncherProfileValidationErrors::InitialMapNotAvailable}),
			},
			SAssignNew(InitalMapCombo, SSearchableComboBox)
			.OptionsSource(&Owner.CachedStartupMaps)
			.OnSelectionChanged( this, &FGenericProfileTreeBuilder::FBuildCookRun::OnInitialMapChanged )
			.OnGenerateWidget( Owner.AsShared(), &FGenericProfileTreeBuilder::OnGenerateComboWidget )
			.OnComboBoxOpening( Owner.AsShared(), &FGenericProfileTreeBuilder::CacheStartupMapList )
			.Content()
			[
				SNew(STextBlock)
				.Text_Lambda( [this]() { return FText::FromString( *BuildCookRun->GetDefaultLaunchRole()->GetInitialMap() ); } )
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			]
		);
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::AddCommandLineProperty( FLaunchProfileTreeNode& HeadingNode )
	{
		HeadingNode.AddCommandLineString( LOCTEXT("CommandLineLabel", "Additional Command Line"), 
			{
				.GetValue = [this]()				{ return GetCommandLine(EBuildTargetType::Unknown); },
				.SetValue = [this](FString Value)	{ SetCommandLine(Value, EBuildTargetType::Unknown); },
				.GetDefaultValue = EmptyString,
				.IsVisible = [this]()				{ return (ContentScheme != EContentScheme::SubmissionPackage); },
				.IsEnabled = ForRun,
				.Validation = FValidation({ELauncherProfileValidationErrors::MalformedLaunchCommandLine}),
			}
		);




		static const struct
		{
			EBuildTargetType TargetType;
			FText Label;
		}
		PerTargetTypeCmdLine[] =
		{
			{ EBuildTargetType::Game,    LOCTEXT("CommandLineLabelGame",   "Game Command Line")   },
			{ EBuildTargetType::Client,  LOCTEXT("CommandLineLabelClient", "Client Command Line") },
			{ EBuildTargetType::Server,  LOCTEXT("CommandLineLabelServer", "Server Command Line") },
			{ EBuildTargetType::Editor,  LOCTEXT("CommandLineLabelEditor", "Editor Command Line") },
		};

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(PerTargetTypeCmdLine); Index++)
		{
			EBuildTargetType TargetType = PerTargetTypeCmdLine[Index].TargetType;

			HeadingNode.AddCommandLineString(  PerTargetTypeCmdLine[Index].Label,
				{
					.GetValue = [this, TargetType]()				{ return GetCommandLine(TargetType); },
					.SetValue = [this, TargetType](FString Value)	{ SetCommandLine(Value, TargetType); },
					.GetDefaultValue = EmptyString,
					.IsVisible = [this, TargetType]()				{ return HasAdvancedPlatformTargets() && CachedBuildTargetTypes.Contains(TargetType); },
					.IsEnabled = ForRun,
				}
			);

		}
	}






















	void FGenericProfileTreeBuilder::FBuildCookRun::OnValidateProfile(ILauncherProfileUATCommandRef UATCommand)
	{
		if (UATCommand != BuildCookRun)
		{
			return;
		}

		if (!Model->AreProjectSettingsReady(Owner.Profile))
		{
			return;
		}

		// verify that IoStore is available when a Zen Workflow is in use
		if ((ContentScheme == EContentScheme::ZenStreaming || ContentScheme == EContentScheme::ZenPakStreaming))
		{
			if (!Model->GetProjectSettings(Owner.Profile).bUseIoStore)
			{
				Owner.Profile->AddCustomError(TEXT("Validation_ZenNeedsIoStore"), LOCTEXT("Error_ZenNeedsIoStore", "Zen Workflows require IoStore to be enabled in your project's Packaging Settings. It cannot be overridden in a profile"), BuildCookRun.ToSharedPtr());
			}
			
			if (!Model->GetProjectSettings(Owner.Profile).bAllowRemoteNetworkService && Model->IsUsingRemotePlatform(BuildCookRun))
			{
				Owner.Profile->AddCustomError(TEXT("Validation_ZenNeedsRemotePermission"), LOCTEXT("Error_ZenNeedsRemotePermission", "Zen Workflows for remote devices need [Zen.AutoLaunch]:RemoteNetworkService to be a non-None value. It cannot be overridden in a profile"), BuildCookRun.ToSharedPtr());
			}
		}

		if (CachedBuildTargetTypes.Contains(EBuildTargetType::Game) && CachedBuildTargetTypes.Contains(EBuildTargetType::Client))
		{
			Owner.Profile->AddCustomError(TEXT("Validation_GameAndClientTargetsDisallowed"), LOCTEXT("Error_NoGameAndClient", "Cannot specify Game and Client targets to be built together"), BuildCookRun.ToSharedPtr());
		}
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::OnProjectSettingsReady(const FString& ProjectPath)
	{
		if (ProjectPath == Owner.Profile->GetProjectPath())
		{
			CacheBuildTargets();
			OnValidateProfile(BuildCookRun);
		}
	}

	void FGenericProfileTreeBuilder::OnUATCommandAdded(const ILauncherProfileUATCommandRef& UATCommand)
	{
		ILauncherProfileBuildCookRunPtr BuildCookRun = UATCommand->AsBuildCookRun();
		if (BuildCookRun.IsValid())
		{
			BuildCookRunData.Add(BuildCookRun.ToSharedRef(), MakeShared<FBuildCookRun>(*this, BuildCookRun.ToSharedRef(), Model));
		}
	}

	void FGenericProfileTreeBuilder::OnUATCommandRemoved(const ILauncherProfileUATCommandRef& UATCommand)
	{
		ILauncherProfileBuildCookRunPtr BuildCookRun = UATCommand->AsBuildCookRun();
		if (BuildCookRun.IsValid())
		{
			BuildCookRunData.Remove(BuildCookRun.ToSharedRef());
		}
	}

	void FGenericProfileTreeBuilder::OnPropertyChanged()
	{
		if (ProfileType == EProfileType::Basic)
		{
			// do not save basic profiles - they're transient
		}
		else
		{
			Model->GetProfileManager()->SaveJSONProfile(Profile);
		}

		TreeData->OnPropertyChanged();

		bStartupMapCacheDirty = true;
	}


	void FGenericProfileTreeBuilder::CacheStartupMapList() const
	{
		if (!bStartupMapCacheDirty)
		{
			return;
		}
		bStartupMapCacheDirty = false;

		CachedStartupMaps.Reset();
		CachedStartupMaps.Add(MakeShared<FString>());

		TSet<FString> AddedMaps;
		for (const FString& Map : Model->GetAvailableProjectMapNames(Profile->GetProjectBasePath()))
		{
			CachedStartupMaps.Add(MakeShared<FString>(Map));
			AddedMaps.Add(Map);
		}

		// Append any maps selected for cooking that aren't project maps
		// (e.g. plugin maps), so they can be set as the Initial Map
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
		{
			for (const FString& CookedMap : BuildCookRun->GetCookedMaps())
			{
				if (!AddedMaps.Contains(CookedMap))
				{
					CachedStartupMaps.Add(MakeShared<FString>(CookedMap));
					AddedMaps.Add(CookedMap);
				}
			}
		}

		// Refresh the combo box's internal filtered list to match the new options
		for (const TPair<ILauncherProfileBuildCookRunRef,TSharedRef<FBuildCookRun>>& Pair : BuildCookRunData)
		{
			if (Pair.Value->InitalMapCombo.IsValid())
			{
				Pair.Value->InitalMapCombo->RefreshOptions();
			}
		}
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::CacheArchitectures()
	{
		CachedBuildTargetArchitectures.Reset();
		CachedBuildTargetArchitectures.Add(EBuildTargetType::Game);
		CachedBuildTargetArchitectures.Add(EBuildTargetType::Client);
		CachedBuildTargetArchitectures.Add(EBuildTargetType::Server);
		CachedBuildTargetArchitectures.Add(EBuildTargetType::Editor);

		// gather all architecures for each target type
		for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : Model->GetPlatformInfos(BuildCookRun))
		{
			const ITargetPlatform* TargetPlatform = GetTargetPlatformManager()->FindTargetPlatform(PlatformInfo->IniPlatformName);
			if (TargetPlatform)
			{
				TArray<FString> PossibleArchitectures;
				TargetPlatform->GetPossibleArchitectures(PossibleArchitectures);

				for (const FString& PossibleArchitecture : PossibleArchitectures)
				{
					CachedBuildTargetArchitectures[PlatformInfo->PlatformType].AddUnique(PossibleArchitecture);
				}
			}
		}

		// insert an empty string at the top of the list for "project default" if there are any architectures
		auto InsertProjectDefault = []( TArray<FString>& Architectures )
		{
			if (Architectures.Num() > 0)
			{
				Architectures.Insert(FString(), 0);
			}
		};
		InsertProjectDefault( CachedBuildTargetArchitectures[EBuildTargetType::Game] );
		InsertProjectDefault( CachedBuildTargetArchitectures[EBuildTargetType::Client] );
		InsertProjectDefault( CachedBuildTargetArchitectures[EBuildTargetType::Server] );
		InsertProjectDefault( CachedBuildTargetArchitectures[EBuildTargetType::Editor] );
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::CacheBuildTargets()
	{
		CachedBuildTargetTypes.Reset();

		if (Model->AreProjectSettingsReady(Owner.Profile))
		{
			TArray<FTargetInfo> TargetInfos = Model->GetBuildTargetInfos(Owner.Profile, BuildCookRun);
			for (const FTargetInfo& TargetInfo : TargetInfos)
			{
				CachedBuildTargetTypes.Add(TargetInfo.Type);
			}

			if (bHasAdvancedPlatformTargets && PlatformCombo.IsValid())
			{
				PlatformCombo->RefreshPlatformsList();
			}
		}
	}



	TSharedRef<SWidget> FGenericProfileTreeBuilder::OnGenerateComboWidget( TSharedPtr<FString> InComboString )
	{
		return SNew(STextBlock)
			.Text(InComboString.IsValid() ? FText::FromString(*InComboString) : FText::GetEmpty() )
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont"));

	}



	void FGenericProfileTreeBuilder::FBuildCookRun::SetSelectedPlatforms( TArray<FString> SelectedPlatforms )
	{
		if (bHasAdvancedPlatformTargets)
		{
			BuildCookRun->ClearCookedPlatforms();
			for (const FString& Platform : SelectedPlatforms)
			{
				BuildCookRun->AddCookedPlatform(Platform);
			}
		}
		else
		{
			BuildCookRun->ClearCookedPlatforms();

			FString BuildTarget = (GetBuildTargets().Num() > 0) ? GetBuildTargets()[0] : FString();
			FTargetInfo BuildTargetInfo = Model->GetBuildTargetInfo(BuildTarget, Owner.GetProjectPath());

			for (const FString& Platform : SelectedPlatforms)
			{
				BuildCookRun->AddCookedPlatform( FModel::GetBuildTargetPlatformName(Platform, BuildTargetInfo));
			}
		}

		if (BuildCookRun->GetDefaultDeployPlatform() != NAME_None)
		{
			FString DeployPlatform = (GetSelectedPlatforms().Num() > 0) ? GetSelectedPlatforms()[0] : FPlatformProperties::IniPlatformName();
			BuildCookRun->SetDefaultDeployPlatform(FName(*DeployPlatform));
		}

		ValidateArchitectures();

		Owner.OnPropertyChanged();

		if (DeployDeviceListView)
		{
			DeployDeviceListView->OnSelectedPlatformChanged();
		}

		CacheArchitectures();
	}

	TArray<FString> FGenericProfileTreeBuilder::FBuildCookRun::GetSelectedPlatforms() const
	{
		TArray<FString> Platforms;

		if (bHasAdvancedPlatformTargets)
		{
			Platforms = BuildCookRun->GetCookedPlatforms();
		}
		else
		{
			for (const FString& Platform : BuildCookRun->GetCookedPlatforms())
			{
				Platforms.Add(FModel::GetPlatformNameWithFlavor(Platform));
			}
		}

		return MoveTemp(Platforms);
	}


	FString FGenericProfileTreeBuilder::GetProjectPath() const
	{
		return Model->GetProjectPath(Profile);
	}

	void FGenericProfileTreeBuilder::SetProjectName(FString ProjectPath)
	{
		Profile->SetProjectPath(ProjectPath);
		Profile->SetProjectSpecified(!ProjectPath.IsEmpty());

		if (Model->AreProjectSettingsReady(Profile))
		{
			ValidateBuildTargets(ProjectPath);
		}

		OnPropertyChanged();
	}

	bool FGenericProfileTreeBuilder::HasProject() const
	{
		return Profile->HasProjectSpecified();
	}

	void FGenericProfileTreeBuilder::ValidateBuildTargets(const FString& InProjectPath)
	{
		if (InProjectPath == Profile->GetProjectPath())
		{
			for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Profile->GetBuildCookRunCommands())
			{
				Model->EnsureValidBuildTarget(Profile, BuildCookRun);

				if (ProfileType != EProfileType::Basic )
				{
					Model->UpdateCookedPlatformsFromBuildTarget(Profile, BuildCookRun);
				}
			}
			OnPropertyChanged();
		}
	}

	TArray<FString> FGenericProfileTreeBuilder::FBuildCookRun::GetBuildTargets() const
	{
		if (Owner.ProfileType == EProfileType::Basic)
		{
			check(!bHasAdvancedPlatformTargets);
			if (BuildCookRun->HasBuildTargetSpecified())
			{
				return BuildCookRun->GetBuildTargets();
			}
			else
			{
				return TArray<FString>{Model->GetProfileManager()->GetBuildTarget()};
			}
		}
		else
		{
			return BuildCookRun->GetBuildTargets();
		}
	}
		
	void FGenericProfileTreeBuilder::FBuildCookRun::SetBuildTargets( TArray<FString> BuildTargets)
	{
		BuildCookRun->ClearBuildTargets();
		if (BuildTargets.Num() == 0 || BuildTargets.Contains(FString()))
		{
			BuildCookRun->SetBuildTargetSpecified(false);
		}
		else
		{
			BuildCookRun->SetBuildTargetSpecified(true);

			for (const FString& BuildTarget : BuildTargets)
			{
				BuildCookRun->AddBuildTarget(BuildTarget);
			}
		}
		CacheBuildTargets();


		if (!bHasAdvancedPlatformTargets)
		{
			if (Owner.ProfileType == EProfileType::Basic )
			{
				Model->UpdatedCookedPlatformsFromDeployDeviceProxy(Owner.Profile, BuildCookRun);
			}
			else
			{
				Model->UpdateCookedPlatformsFromBuildTarget(Owner.Profile, BuildCookRun);
			}
		}

		ValidateArchitectures();

		Owner.OnPropertyChanged();
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::SetBuildConfiguration(EBuildConfiguration BuildConfiguration)
	{
		BuildCookRun->SetBuildConfiguration(BuildConfiguration);
		Owner.OnPropertyChanged();
	}

	EBuildConfiguration FGenericProfileTreeBuilder::FBuildCookRun::GetBuildConfiguration() const
	{
		return BuildCookRun->GetBuildConfiguration();
	}



	void FGenericProfileTreeBuilder::FBuildCookRun::RefreshContentScheme()
	{
		EContentScheme CurrentContentScheme = Model->DetermineProfileContentScheme(BuildCookRun);
		SetContentScheme(CurrentContentScheme);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::SetContentScheme(EContentScheme InContentScheme)
	{
		FProjectSettings ProjectSettings = Model->GetProjectSettings(Owner.Profile);

		ELauncherProfileDeploymentModes::Type DeploymentMode = GetDeployToDevice() ? ELauncherProfileDeploymentModes::CopyToDevice : ELauncherProfileDeploymentModes::DoNotDeploy;
		Model->SetProfileContentScheme(InContentScheme, ProjectSettings, BuildCookRun, bShouldCook, DeploymentMode );

		// refresh the cached content scheme in case the option that was selected is not available
		ContentScheme = Model->DetermineProfileContentScheme(BuildCookRun);

		Owner.OnPropertyChanged();
	}

	bool FGenericProfileTreeBuilder::FBuildCookRun::IsContentSchemeAvailable(EContentScheme InContentScheme, FText& OutReason) const
	{
		FProjectSettings ProjectSettings = Model->GetProjectSettings(Owner.Profile);

		// basic launch is aimed at launching the current content, not an external build
		if (Owner.ProfileType == EProfileType::Basic && InContentScheme == EContentScheme::ZenPakStreaming)
		{
			// don't set a reason - just hide the item for Basic Launch
			return false;
		}

		// cannot create a submission package from the Basic Launch
		if (Owner.ProfileType == EProfileType::Basic && InContentScheme == EContentScheme::SubmissionPackage)
		{
			// don't set a reason - just hide the item for Basic Launch
			return false;
		}

		// loose files can't be selected if the project is using zen store because there's no way to opt-out of Zen Store from the UAT command line.
		if (InContentScheme == EContentScheme::LooseFiles && ProjectSettings.bUseZenStore)
		{
			OutReason = LOCTEXT("NoLooseFilesReason", "Loose Files cannot be used when Zen Store is enabled in Project Settings");
			return false;
		}

		// don't show zen pak streaming option if it isn't going to be set up automatically by UAT
		// @todo: could potentially look at shelling out to Zen & querying if we have a dynamic workspace? no support for async config queries in this tool yet though
		if (InContentScheme == EContentScheme::ZenPakStreaming && !ProjectSettings.bHasAutomaticZenPakStreamingWorkspaceCreation)
		{
			OutReason = LOCTEXT("NoZenPakReason", "Automatic Zen Pak streaming workspace creation has not been enabled in Project Settings");
			return false;
		}
	
		// cannot launch via Zen if we're targeting a remote device and Zen isn't accepting external connections
		if ((InContentScheme == EContentScheme::ZenStreaming || InContentScheme == EContentScheme::ZenPakStreaming) 
			&& !ProjectSettings.bAllowRemoteNetworkService && Model->IsUsingRemotePlatform(BuildCookRun))
		{
			OutReason = LOCTEXT("NoZenReason", "Zen Streaming to a remote device requires RemoteNetworkService");
			return false;
		}

		// cannot make a submission package outside of the shipping configuration
		if (InContentScheme == EContentScheme::SubmissionPackage && BuildCookRun->GetBuildConfiguration() != EBuildConfiguration::Shipping)
		{
			OutReason = LOCTEXT("SubPkgShippingOnly", "Submission packages require a Shipping build");
			return false;
		}

		return true;
	}



	FString FGenericProfileTreeBuilder::FBuildCookRun::GetCommandLine(EBuildTargetType BuildTarget) const
	{
		if (BuildTarget == EBuildTargetType::Unknown)
		{
			// get unified command line from these two fields. the first is presented in "Build" for old Project Launcher, and the latter is presented in "Launch" for old Project Launcher)
			// when we save back to the profile, this will be stored just in the "Build" one for clarity (because multiple roles are not supported in old or new project launcher)
			FString CommandLine = BuildCookRun->GetAdditionalCommandLineParameters().TrimStartAndEnd() + TEXT(" ") + BuildCookRun->GetDefaultLaunchRole()->GetUATCommandLine().TrimStartAndEnd();
			CommandLine.TrimStartAndEndInline();

			return CommandLine;
		}
		else
		{
			return BuildCookRun->GetAdditionalTargetCommandLineParameters(BuildTarget).TrimStartAndEnd();
		}
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::SetCommandLine( const FString& NewCommandLine, EBuildTargetType BuildTarget )
	{
		if (BuildTarget == EBuildTargetType::Unknown)
		{
			BuildCookRun->SetAdditionalCommandLineParameters(NewCommandLine);
			BuildCookRun->GetDefaultLaunchRole()->SetCommandLine(TEXT(""));
		}
		else
		{
			BuildCookRun->SetAdditionalTargetCommandLineParameters(NewCommandLine, BuildTarget);
		}
		Owner.OnPropertyChanged();
	}







	void FGenericProfileTreeBuilder::FBuildCookRun::SetCook( bool bCook )
	{
		bShouldCook = bCook;
		RefreshContentScheme();
	}

	bool FGenericProfileTreeBuilder::FBuildCookRun::GetCook(ILauncherProfileBuildCookRunPtr InBuildCookRun) const
	{
		if (InBuildCookRun == nullptr)
		{
			InBuildCookRun = BuildCookRun;
		}
		return (InBuildCookRun->GetCookMode() != ELauncherProfileCookModes::DoNotCook);
	}






	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	TSharedRef<SWidget> FGenericProfileTreeBuilder::FBuildCookRun::CreateDeployDeviceWidget()
	{
		if (Owner.ProfileType == EProfileType::Basic)
		{
			return SNew(SCustomLaunchDeviceListView)
				.OnSelectionChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetDeployDeviceIDs)
				.SelectedDevices(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetDeployDeviceIDs)
				.AllPlatforms(true)
				.SingleSelect(true)
			;
		}
		else
		{
			return SNew(SVerticalBox)

				// device picker options
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0,2)
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.Padding(0, 0)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SSegmentedControl<EDeployDeviceOption>)
						.Value( this, &FGenericProfileTreeBuilder::FBuildCookRun::GetDeployDeviceOption)
						.OnValueChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetDeployDeviceOption)

						+SSegmentedControl<EDeployDeviceOption>::Slot(EDeployDeviceOption::Default)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DefaultDeviceLabel", "Default Device"))
							.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
						]

						+SSegmentedControl<EDeployDeviceOption>::Slot(EDeployDeviceOption::Selected)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SelectedDevicesLabel", "Selected Devices"))
							.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
						]
					]
				]

				// device picker list
				+SVerticalBox::Slot()
				.FillHeight(1)
				[
					SNew(SVerticalResizeBox)
					.Visibility_Lambda([this](){ return (GetDeployDeviceOption() == EDeployDeviceOption::Selected) ? EVisibility::Visible : EVisibility::Collapsed; } )
					.HandleHeight(4)
					.ContentHeight(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetDeployDeviceListHeight)
					.ContentHeightChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetDeployDeviceListHeight)
					.HandleColor( FAppStyle::Get().GetSlateColor("Colors.Secondary").GetSpecifiedColor() )
					[
						SAssignNew(DeployDeviceListView, SCustomLaunchDeviceListView)
						.OnSelectionChanged(this, &FGenericProfileTreeBuilder::FBuildCookRun::SetDeployDeviceIDs)
						.SelectedDevices(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetDeployDeviceIDs)
						.Platforms(this, &FGenericProfileTreeBuilder::FBuildCookRun::GetSelectedPlatforms)
					]
				]
			;
		}
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION


	void FGenericProfileTreeBuilder::FBuildCookRun::SetDeployDeviceIDs(TArray<FString> DeployDeviceIDs)
	{
		if (BuildCookRun->GetDeployedDeviceGroup() == nullptr)
		{
			BuildCookRun->SetDeployedDeviceGroup(Model->GetProfileManager()->AddNewDeviceGroup());
		}

		BuildCookRun->GetDeployedDeviceGroup()->RemoveAllDevices(); 
		for ( const FString& DeviceID : DeployDeviceIDs )
		{
			BuildCookRun->GetDeployedDeviceGroup()->AddDevice(DeviceID);
		}

		if (Owner.ProfileType == EProfileType::Basic)
		{
			Model->UpdatedCookedPlatformsFromDeployDeviceProxy(Owner.Profile, BuildCookRun);
		}

		Owner.OnPropertyChanged();
	}

	TArray<FString> FGenericProfileTreeBuilder::FBuildCookRun::GetDeployDeviceIDs() const
	{
		return BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs();
	}

	float FGenericProfileTreeBuilder::FBuildCookRun::GetDeployDeviceListHeight() const
	{
		return DeployDeviceListHeight;
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::SetDeployDeviceListHeight( float NewHeight )
	{
		static const float MinDeployDeviceListHeight = 100.0f;

		DeployDeviceListHeight = FMath::Max(NewHeight, MinDeployDeviceListHeight);

		Owner.TreeData->RequestTreeRefresh();
	}

	FGenericProfileTreeBuilder::FBuildCookRun::EDeployDeviceOption FGenericProfileTreeBuilder::FBuildCookRun::GetDeployDeviceOption() const
	{
		return (BuildCookRun->GetDefaultDeployPlatform() == NAME_None) ? EDeployDeviceOption::Selected : EDeployDeviceOption::Default;;
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::SetDeployDeviceOption( EDeployDeviceOption NewDeployDeviceOption )
	{
		bool bShow = (NewDeployDeviceOption == EDeployDeviceOption::Selected);

		if (bShow)
		{
			BuildCookRun->SetDefaultDeployPlatform(NAME_None);

			// restore the deployed device list again, if any
			if (CachedDeployDeviceIDs.Num() > 0 && BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs().Num() == 0)
			{
				SetDeployDeviceIDs(BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs());
				CachedDeployDeviceIDs.Reset();
			}
		}
		else
		{
			// to set the 'default' deploy option, its necessary to remove all the devices - take a copy of the values to allow it to be restored
			CachedDeployDeviceIDs = BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs();
			SetDeployDeviceIDs(TArray<FString>());

			FString DeployPlatform = (GetSelectedPlatforms().Num() > 0) ? GetSelectedPlatforms()[0] : FPlatformProperties::IniPlatformName();
			BuildCookRun->SetDefaultDeployPlatform(FName(*DeployPlatform));
		}

		Owner.OnPropertyChanged();

		if (DeployDeviceListView)
		{
			DeployDeviceListView->RefreshDeviceList();
		}
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::SetBuild( bool bBuild )
	{
		if (!bBuild)
		{
			BuildCookRun->SetBuildMode(ELauncherProfileBuildModes::DoNotBuild);
		}
		else if (GetForceBuild(BuildCookRun))
		{
			BuildCookRun->SetBuildMode(ELauncherProfileBuildModes::Build);
		}
		else
		{
			BuildCookRun->SetBuildMode(ELauncherProfileBuildModes::Auto);
		}

		Owner.OnPropertyChanged();
	}

	bool FGenericProfileTreeBuilder::FBuildCookRun::GetBuild(ILauncherProfileBuildCookRunPtr InBuildCookRun) const
	{
		if (InBuildCookRun == nullptr)
		{
			InBuildCookRun = BuildCookRun;
		}
		return (InBuildCookRun->GetBuildMode() != ELauncherProfileBuildModes::DoNotBuild);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::SetForceBuild( bool bForceBuild )
	{
		if (!GetBuild())
		{
			BuildCookRun->SetBuildMode(ELauncherProfileBuildModes::DoNotBuild);
		}
		else if (bForceBuild)
		{
			BuildCookRun->SetBuildMode(ELauncherProfileBuildModes::Build);
		}
		else
		{
			BuildCookRun->SetBuildMode(ELauncherProfileBuildModes::Auto);
		}

		Owner.OnPropertyChanged();
	}

	bool FGenericProfileTreeBuilder::FBuildCookRun::GetForceBuild(ILauncherProfileBuildCookRunPtr InBuildCookRun) const
	{
		if (InBuildCookRun == nullptr)
		{
			InBuildCookRun = BuildCookRun;
		}
		return (InBuildCookRun->GetBuildMode() == ELauncherProfileBuildModes::Build);
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::SetArchitecture( FString Architecture, EBuildTargetType BuildTargetType )
	{
		// Project Launcher only supports launching for one architecture at a time at the moment
		TArray<FString> Architectures;
		if (!Architecture.IsEmpty())
		{
			Architectures.Add(Architecture);
		}

		// update the architecture
		if (BuildTargetType == EBuildTargetType::Game || BuildTargetType == EBuildTargetType::Client)
		{
			BuildCookRun->SetClientArchitectures(Architectures);
		}
		else if (BuildTargetType == EBuildTargetType::Server)
		{
			BuildCookRun->SetServerArchitectures(Architectures);
		}
		else if (BuildTargetType == EBuildTargetType::Editor)
		{
			BuildCookRun->SetEditorArchitectures(Architectures);
		}

		Owner.OnPropertyChanged();
	}

	FString FGenericProfileTreeBuilder::FBuildCookRun::GetArchitecture(EBuildTargetType BuildTargetType) const
	{
		// read current architectures
		TArray<FString> Architectures;
		switch (BuildTargetType)
		{
			case EBuildTargetType::Server: Architectures = BuildCookRun->GetServerArchitectures(); break;
			case EBuildTargetType::Editor: Architectures = BuildCookRun->GetEditorArchitectures(); break;
			default:                       Architectures = BuildCookRun->GetClientArchitectures(); break;
		}

		// Project Launcher only supports launching for one architecture at a time at the moment
		return (Architectures.Num() > 0) ? Architectures[0] : FString();
	}

	FText FGenericProfileTreeBuilder::FBuildCookRun::GetArchitectureDisplayName( FString Architecture )
	{
		if (Architecture.IsEmpty())
		{
			return LOCTEXT("DefaultArchName", "Project Default");
		}
		else if (Architecture == FPlatformMisc::GetHostArchitecture())
		{
			return FText::Format(LOCTEXT("HostArchLabel", "{0} (this platform)"), FText::FromString(Architecture) );
		}
		else
		{
			return FText::FromString(Architecture);
		}
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::ValidateArchitectures()
	{
		CacheArchitectures();

		// if the current build targets don't support architectures, remove the

		if (CachedBuildTargetArchitectures[EBuildTargetType::Game].Num() == 0 && CachedBuildTargetArchitectures[EBuildTargetType::Client].Num() == 0)
		{
			BuildCookRun->SetClientArchitectures(TArray<FString>());
		}
		if (CachedBuildTargetArchitectures[EBuildTargetType::Server].Num() == 0)
		{
			BuildCookRun->SetServerArchitectures(TArray<FString>());
		}
		if (CachedBuildTargetArchitectures[EBuildTargetType::Editor].Num() == 0)
		{
			BuildCookRun->SetEditorArchitectures(TArray<FString>());
		}
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::SetDeployToDevice( bool bDeployToDevice )
	{
		if (ContentScheme != EContentScheme::CookOnTheFly)
		{
			BuildCookRun->SetDeploymentMode(bDeployToDevice ? ELauncherProfileDeploymentModes::CopyToDevice : ELauncherProfileDeploymentModes::DoNotDeploy );
			Owner.OnPropertyChanged();
		}
	}

	bool FGenericProfileTreeBuilder::FBuildCookRun::GetDeployToDevice(ILauncherProfileBuildCookRunPtr InBuildCookRun) const
	{
		if (InBuildCookRun == nullptr)
		{
			InBuildCookRun = BuildCookRun;
		}
		return InBuildCookRun->GetDeploymentMode() != ELauncherProfileDeploymentModes::DoNotDeploy;
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::SetIsRunning( bool bRun )
	{
		BuildCookRun->SetLaunchMode(bRun ? ELauncherProfileLaunchModes::DefaultRole : ELauncherProfileLaunchModes::DoNotLaunch);
	}

	bool FGenericProfileTreeBuilder::FBuildCookRun::GetIsRunning( ILauncherProfileBuildCookRunPtr InBuildCookRun) const
	{
		if (InBuildCookRun == nullptr)
		{
			InBuildCookRun = BuildCookRun;
		}
	
		return InBuildCookRun->GetLaunchMode() != ELauncherProfileLaunchModes::DoNotLaunch;
	}

	void FGenericProfileTreeBuilder::FBuildCookRun::OnInitialMapChanged( TSharedPtr<FString> InitialMap, ESelectInfo::Type )
	{
		if (InitialMap.IsValid())
		{
			BuildCookRun->GetDefaultLaunchRole()->SetInitialMap(*InitialMap);
		}
		else
		{
			BuildCookRun->GetDefaultLaunchRole()->SetInitialMap(FString());
		}

		Owner.OnPropertyChanged();
	}

	TSharedPtr<FString> FGenericProfileTreeBuilder::FBuildCookRun::GetInitialMap() const
	{
		Owner.CacheStartupMapList();

		FString InitialMap = BuildCookRun->GetDefaultLaunchRole()->GetInitialMap();
		const TSharedPtr<FString>* FoundMapPtr = Owner.CachedStartupMaps.FindByPredicate( [InitialMap]( const TSharedPtr<FString>& Map )
		{
			return Map.IsValid() && (InitialMap == *Map);
		});

		if (FoundMapPtr)
		{
			return *FoundMapPtr;
		}
		else
		{
			return MakeShared<FString>();
		}
	}


	void FGenericProfileTreeBuilder::FBuildCookRun::SetHasAdvancedPlatformTargets( bool bEnable )
	{
		if (bHasAdvancedPlatformTargets && !bEnable)
		{
			if (BuildCookRun->GetCookedPlatforms().Num() > 1 || BuildCookRun->GetBuildTargets().Num() > 1)
			{
				FText MessageText = LOCTEXT("ClearAdvancedPlatformTargets", "You have multiple platform/targets selected - are you sure you want to go back to single platform/target??");
				if (!ProjectLauncher::GetUserConfirmation(MessageText, ProjectLauncher::ProjectLauncherText, false))
				{
					return;
				}
			}

			bHasAdvancedPlatformTargets = bEnable;

			if (BuildCookRun->GetBuildTargets().Num() > 1)
			{
				FString BuildTarget = BuildCookRun->GetBuildTargets()[0];
				BuildCookRun->ClearBuildTargets();
				BuildCookRun->AddBuildTarget(BuildTarget);

			}
			Model->UpdateCookedPlatformsFromBuildTarget(Owner.Profile, BuildCookRun);

			if (BuildCookRun->GetCookedPlatforms().Num() > 0)
			{
				TArray<FString> Platforms{BuildCookRun->GetCookedPlatforms()[0]};
				SetSelectedPlatforms(Platforms);
			}

			CacheBuildTargets();
		}
		else
		{
			bHasAdvancedPlatformTargets = bEnable;
			Owner.OnPropertyChanged();
		}

		if (PlatformCombo)
		{
			PlatformCombo->RefreshPlatformsList();
		}
	}

	bool FGenericProfileTreeBuilder::FBuildCookRun::HasAdvancedPlatformTargets() const
	{
		return bHasAdvancedPlatformTargets;
	}


	FGenericProfileTreeBuilder::FBuildCookRun& FGenericProfileTreeBuilder::Get(const ILauncherProfileBuildCookRunRef& BuildCookRun)
	{
		return BuildCookRunData.FindChecked(BuildCookRun).Get();
	}

}

#undef LOCTEXT_NAMESPACE
