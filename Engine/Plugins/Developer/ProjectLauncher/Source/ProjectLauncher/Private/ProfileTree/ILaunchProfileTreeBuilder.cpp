// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileTree/ILaunchProfileTreeBuilder.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"

namespace ProjectLauncher
{
	static TArray<TSharedPtr<ILaunchProfileTreeBuilderFactory>> GTreeBuilderFactories;
	static FCriticalSection GTreeBuilderFactoriesCS;



	class FEmptyProfileTreeBuilder : public ILaunchProfileTreeBuilder
	{
	public:
		FEmptyProfileTreeBuilder(const ILauncherProfilePtr& InProfile, TSharedRef<FModel> InModel)
			: TreeData(MakeShared<FLaunchProfileTreeData>(InProfile, InModel, this))
		{
		}
		virtual ~FEmptyProfileTreeBuilder() = default;

		virtual void Construct() override
		{
		}

		virtual FString GetName() const override
		{
			return TEXT("Empty");
		}

		virtual FLaunchProfileTreeDataRef GetProfileTree() override
		{
			return TreeData;
		}

		virtual void OnPropertyChanged() override
		{
		}

		virtual bool AllowExtensionsUI() const override
		{
			return false;
		}

	protected:
		FLaunchProfileTreeDataRef TreeData;
	};





	void RegisterTreeBuilderFactory( TSharedRef<ILaunchProfileTreeBuilderFactory> TreeBuilderFactory )
	{
		FScopeLock Lock(&GTreeBuilderFactoriesCS);

		GTreeBuilderFactories.Add(TreeBuilderFactory);
		GTreeBuilderFactories.Sort( []( TSharedPtr<ILaunchProfileTreeBuilderFactory> A, TSharedPtr<ILaunchProfileTreeBuilderFactory> B)
		{
			return A->GetPriority() > B->GetPriority();
		});

	}

	void UnregisterTreeBuilderFactory( TSharedRef<ILaunchProfileTreeBuilderFactory> TreeBuilderFactory )
	{
		FScopeLock Lock(&GTreeBuilderFactoriesCS);

		GTreeBuilderFactories.Remove(TreeBuilderFactory);
	}


	TSharedRef<ILaunchProfileTreeBuilder> CreateTreeBuilder( const ILauncherProfilePtr& InProfile, TSharedRef<FModel> InModel )
	{
		if (InProfile.IsValid())
		{
			EProfileType ProfileType = InModel->GetProfileType(InProfile.ToSharedRef());

			FScopeLock Lock(&GTreeBuilderFactoriesCS);

			// find a tree builder that can handle this profile
			for (const TSharedPtr<ILaunchProfileTreeBuilderFactory>& Factory : GTreeBuilderFactories)
			{
				if (!Factory->IsProfileTypeSupported(ProfileType))
				{
					continue;
				}

				TSharedPtr<ILaunchProfileTreeBuilder> TreeBuilder = Factory->TryCreateTreeBuilder(InProfile.ToSharedRef(), InModel);
				if (TreeBuilder.IsValid())
				{
					TreeBuilder->Construct();

					if (TreeBuilder->AllowExtensionsUI())
					{
						TreeBuilder->GetProfileTree()->CreateExtensionsUI();
					}

					return TreeBuilder.ToSharedRef();
				}
			}
		}

		// cannot build tree
		TSharedRef<ILaunchProfileTreeBuilder> EmptyTreeBuilder = MakeShared<FEmptyProfileTreeBuilder>(InProfile, InModel);
		EmptyTreeBuilder->Construct();
		return EmptyTreeBuilder;
	}
}

