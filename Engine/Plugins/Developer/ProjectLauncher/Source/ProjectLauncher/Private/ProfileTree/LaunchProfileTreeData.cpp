// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileTree/LaunchProfileTreeData.h"
#include "ProfileTree/ILaunchProfileTreeBuilder.h"
#include "Extension/LaunchExtension.h"
#include "Extension/CmdLineParametersExtension.h"
#include "Utils/LauncherCmdLineUtils.h"
#include "Styling/ProjectLauncherStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Layout/Visibility.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UIAction.h"
#include "DesktopPlatformModule.h"
#include "Misc/ConfigCacheIni.h"


#define LOCTEXT_NAMESPACE "SCustomLaunchCustomProfileEditor"

namespace ProjectLauncher
{
	FValidation::FValidation(std::initializer_list<ELauncherProfileValidationErrors::Type> InAssociatedErrors )
		: AssociatedErrors(InAssociatedErrors)
	{
		bIsSet = true;
	}

	FValidation::FValidation(std::initializer_list<ELauncherProfileValidationErrors::Type> InAssociatedErrors, std::initializer_list<FString> InAssociatedCustomErrors )
		: AssociatedErrors(InAssociatedErrors)
		, AssociatedCustomErrors(InAssociatedCustomErrors)
	{
		bIsSet = true;
	}

	FValidation::FValidation( std::initializer_list<FString> InAssociatedCustomErrors )
		: AssociatedCustomErrors(InAssociatedCustomErrors)
	{
		bIsSet = true;
	}


	bool FValidation::IsSet() const
	{
		return bIsSet;
	}

	bool FValidation::HasError( ILauncherProfileRef Profile, ILauncherProfileUATCommandPtr UATCommand ) const
	{
		if (Profile->HasValidationError(UATCommand))
		{
			for (ELauncherProfileValidationErrors::Type Error : AssociatedErrors)
			{
				if (Profile->HasValidationError(Error, UATCommand))
				{
					return true;
				}
			}
		}

		for (const FString& CustomError : AssociatedCustomErrors)
		{
			if (Profile->HasCustomError(CustomError, UATCommand))
			{
				return true;
			}
		}

		for (const FString& CustomError : AssociatedCustomErrors)
		{
			if (Profile->HasCustomWarning(CustomError, UATCommand))
			{
				return true;
			}
		}

		return false;
	}

	void FValidation::GetErrorText( ILauncherProfileRef Profile, ILauncherProfileUATCommandPtr UATCommand, TArray<FString>& ErrorLines ) const
	{
		for (ELauncherProfileValidationErrors::Type Error : AssociatedErrors)
		{
			if (Profile->HasValidationError(Error, UATCommand))
			{
				ErrorLines.AddUnique(LexToStringLocalized(Error));
			}
		}

		for (const FString& CustomError : AssociatedCustomErrors)
		{
			if (Profile->HasCustomError(CustomError, UATCommand))
			{
				ErrorLines.AddUnique(Profile->GetCustomErrorText(CustomError, UATCommand).ToString());
			}
		}

		for (const FString& CustomWarning : AssociatedCustomErrors)
		{
			if (Profile->HasCustomWarning(CustomWarning, UATCommand))
			{
				ErrorLines.AddUnique(Profile->GetCustomWarningText(CustomWarning, UATCommand).ToString());
			}
		}
	}

	const TCHAR* FLaunchProfileTreeData::GeneralSettingsSectionName = TEXT("GeneralSettings");
	const int32 FLaunchProfileTreeData::GeneralSettingsSortOrder = INT32_MIN;
	const int32 FLaunchProfileTreeData::DefaultSortOrder = INT32_MIN+1;

	FLaunchProfileTreeData::FLaunchProfileTreeData(ILauncherProfilePtr InProfile, TSharedRef<ProjectLauncher::FModel> InModel, ILaunchProfileTreeBuilder* InTreeBuilder) 
		: Profile(InProfile)
		, Model(InModel)
		, TreeBuilder(InTreeBuilder)
	{
		check(TreeBuilder);

		if (InProfile.IsValid())
		{
			ExtensionInstances = FLaunchExtension::CreateExtensionInstancesForProfile(Profile.ToSharedRef(), InModel, *this);

			for (const TSharedPtr<FLaunchExtensionInstance>& ExtensionInstance : ExtensionInstances)
			{
				ExtensionInstance->GetPropertyChangedDelegate().AddRaw(TreeBuilder, &ILaunchProfileTreeBuilder::OnPropertyChanged );
			}
		}
	}

	FLaunchProfileTreeData::~FLaunchProfileTreeData()
	{
		FLaunchExtension::DestroyExtensionInstancesForProfile(Profile);
		ExtensionInstances.Reset();
	}


	FLaunchProfileTreeNode& FLaunchProfileTreeData::AddHeading( const TCHAR* InName, FText InDisplayName, int32 InSortOrder )
	{
		if (HeadingNodes.Contains(InName))
		{
			return GetHeading(InName);
		}

		FLaunchProfileTreeNodePtr TreeNode = MakeShared<FLaunchProfileTreeNode>(this);
		TreeNode->Name = InDisplayName;
		TreeNode->SortOrder = InSortOrder;
		TreeNode->Depth = 0;
		Nodes.Add(TreeNode);
		RequestExpandHeadings.Add(TreeNode);


		HeadingNodes.Add(InName, TreeNode);

		SortNodes();

		return *TreeNode.Get();
	}

	FLaunchProfileTreeNode& FLaunchProfileTreeData::GetHeading( const TCHAR* InName ) const
	{
		FLaunchProfileTreeNodePtr TreeNode = HeadingNodes.FindChecked(InName);
		return *TreeNode.Get();
	}

	FLaunchProfileTreeNode& FLaunchProfileTreeData::FindOrAddHeading(ILauncherProfileUATCommandRef InUATCommand, bool bIsPrimaryHeading)
	{
		if (HeadingNodes.Contains(InUATCommand->GetInternalName()))
		{
			return GetHeading(InUATCommand->GetInternalName());
		}

		FLaunchProfileTreeNode& TreeNode = AddHeading(InUATCommand->GetInternalName(), FText::FromString(*InUATCommand->GetDescription()), InUATCommand->GetOrder()); // @todo: need description to auto-update
		TreeNode.UATCommand = InUATCommand;

		if (bIsPrimaryHeading)
		{
			UATCommandHeadings.Add(TreeNode.AsShared());
		}

		return TreeNode;
	}

	void FLaunchProfileTreeData::SortNodes()
	{
		Nodes.StableSort( []( const FLaunchProfileTreeNodePtr& A, const FLaunchProfileTreeNodePtr& B)
		{
			return A->SortOrder < B->SortOrder;
		});
	}

	void FLaunchProfileTreeData::CreateExtensionsUI()
	{
		for (TSharedPtr<FLaunchExtensionInstance> ExtensionInstance : ExtensionInstances)
		{
			ExtensionInstance->CustomizeTree();
		}
	}
	
	void FLaunchProfileTreeData::RequestTreeRefresh()
	{
		bRequestTreeRefresh = true;
		Profile->RefreshCustomWarningsAndErrors();
	}

	void FLaunchProfileTreeData::RequestFullTreeRebuild()
	{
		bRequestTreeFullRebuild = true;
	}

	void FLaunchProfileTreeData::OnLaunchExtensionToggle( const TSharedRef<FLaunchExtensionInstance>& ExtensionInstance, bool bAdded )
	{
		ProjectLauncher::ICmdLineParametersExtensionFactory* CmdLineFactory = ExtensionInstance->AsCmdLineParametersFactory();
		if (CmdLineFactory)
		{
			FString TypeName = CmdLineFactory->GetCmdLineType();

			// toggle the default command line parameters for all matching command line properties
			for ( const FCommandLineField& CommandLineField : AllCommandLineFields )
			{
				if (CommandLineField.TypeName == TypeName)
				{
					// get the relevant parameters for this extension
					static const bool bOnlyRemoveDefaultCmdLineParameters = false;

					TArray<FString> Parameters;
					if (bAdded || bOnlyRemoveDefaultCmdLineParameters)
					{
						CmdLineFactory->GetDefaultCmdLineParameters( Parameters );
					}
					else
					{
						CmdLineFactory->GetCmdLineParameters( Parameters );
					}

					// toggle them all
					if (Parameters.Num() > 0)
					{
						FString CommandLine = CommandLineField.GetValue();
						for (const FString& Parameter : Parameters)
						{
							if (bAdded)
							{
								CmdLineUtils::AddParameter(CommandLine, Parameter);
							}
							else
							{
								CmdLineUtils::RemoveParameter(CommandLine, Parameter);
							}
						}
						CommandLineField.SetValue(CommandLine);
					}
				}
			}
		}

		if (bAdded)
		{
			RequestFullTreeRebuild();
		}
	}

	void FLaunchProfileTreeData::OnCommandLineFieldCreated( FLaunchProfileTreeNode& InNode, const FLaunchProfileTreeNode::FStringCallbacks& InStringCallbacks, const FString& InTypeName )
	{
		// store the command line callback
		AllCommandLineFields.Emplace( FLaunchProfileTreeData::FCommandLineField 
		{ 
			.GetValue = InStringCallbacks.GetValue, 
			.SetValue = InStringCallbacks.SetValue,
			.TypeName = InTypeName,
		});
	}

	void FLaunchProfileTreeData::OnPropertyChanged()
	{
		for (TSharedPtr<FLaunchExtensionInstance> ExtensionInstance : ExtensionInstances)
		{
			ExtensionInstance->OnPropertyChanged();
		}

		RequestTreeRefresh();
	}



	FLaunchProfileTreeNode::FLaunchProfileTreeNode( FLaunchProfileTreeData* InTreeData ) 
		: TreeData(InTreeData)
	{
	}

	void FLaunchProfileTreeNode::OnPropertyChanged()
	{
		TreeData->TreeBuilder->OnPropertyChanged();
	}

	bool FLaunchProfileTreeNode::ContainsNodeRecursive( const FLaunchProfileTreeNodePtr& TargetNode ) const
	{
		if (TargetNode.Get() == this)
		{
			return true;
		}

		for (const ProjectLauncher::FLaunchProfileTreeNodePtr& Child : Children)
		{
			if (Child->ContainsNodeRecursive(TargetNode))
			{
				return true;
			}
		}

		return false;
	}


	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddSubHeading( const TCHAR* InName, FText InDisplayName, int32 InSortOrder )
	{
		if (SubHeadingNodes.Contains(InName))
		{
			return GetSubHeading(InName);
		}

		FLaunchProfileTreeNodePtr TreeNode = MakeShared<FLaunchProfileTreeNode>(TreeData);
		TreeNode->Name = InDisplayName;
		TreeNode->SortOrder = InSortOrder;
		TreeNode->Depth = Depth+1;
		TreeNode->UATCommand = UATCommand;
		Children.Add(TreeNode);
		TreeData->RequestExpandHeadings.Add(TreeNode);

		SubHeadingNodes.Add(InName, TreeNode);

		Children.StableSort( []( const FLaunchProfileTreeNodePtr& A, const FLaunchProfileTreeNodePtr& B)
		{
			return A->SortOrder < B->SortOrder;
		});

		return *TreeNode.Get();
	}


	FLaunchProfileTreeNode& FLaunchProfileTreeNode::GetSubHeading(const TCHAR* InName) const
	{
		FLaunchProfileTreeNodePtr TreeNode = SubHeadingNodes.FindChecked(InName);
		return *TreeNode.Get();
	}

	FLaunchProfileTreeNode& FLaunchProfileTreeNode::FindOrAddSubHeading(ILauncherProfileUATCommandRef InUATCommand)
	{
		if (SubHeadingNodes.Contains(InUATCommand->GetInternalName()))
		{
			return GetSubHeading(InUATCommand->GetInternalName());
		}

		FLaunchProfileTreeNode& TreeNode = AddSubHeading(InUATCommand->GetInternalName(), FText::FromString(*InUATCommand->GetDescription()), InUATCommand->GetOrder()); // @todo: need description and sort order to auto-update
		TreeNode.UATCommand = InUATCommand;

		return TreeNode;
	}

	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddWidget( FText InName, FCallbacks&& InWidgetCallbacks, TSharedRef<SWidget> InValueWidget )
	{
		FLaunchProfileTreeNodePtr TreeNode = MakeShared<FLaunchProfileTreeNode>(TreeData);
		TreeNode->Name = InName;
		TreeNode->SortOrder = 0;
		TreeNode->Depth = Depth+1;
		TreeNode->Widget = InValueWidget;
		TreeNode->Callbacks = InWidgetCallbacks;
		TreeNode->UATCommand = UATCommand;
		Children.Add(TreeNode);

		Children.StableSort( []( const FLaunchProfileTreeNodePtr& A, const FLaunchProfileTreeNodePtr& B)
		{
			return A->SortOrder < B->SortOrder;
		});


		TreeData->RequestTreeRefresh();
		return *this;
	}

	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddWidget( FText InName, TSharedRef<SWidget> InValueWidget )
	{
		return AddWidget( InName, {}, InValueWidget );
	}


	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddBoolean( FText InName, FBooleanCallbacks&& BooleanCallbacks, FText ToolTipText)
	{
		check(BooleanCallbacks.GetValue);
		check(BooleanCallbacks.SetValue);

		FCallbacks WidgetCallbacks
		{
			.IsVisible = BooleanCallbacks.IsVisible,
			.Validation = BooleanCallbacks.Validation,
		};
		if (BooleanCallbacks.GetDefaultValue != nullptr)
		{
			WidgetCallbacks.IsDefault = [BooleanCallbacks]()
			{
				return BooleanCallbacks.GetValue() == BooleanCallbacks.GetDefaultValue();
			};

			WidgetCallbacks.SetToDefault = [BooleanCallbacks]()
			{
				BooleanCallbacks.SetValue(BooleanCallbacks.GetDefaultValue());
				// nb. OnPropertyChanged is called via SCustomLaunchWidgetTableRow's set to default
			};
		};

		auto SetCheckState = [this, BooleanCallbacks]( ECheckBoxState CheckState )
		{
			BooleanCallbacks.SetValue( (CheckState == ECheckBoxState::Checked) );
			OnPropertyChanged();
		};

		auto GetCheckState = [BooleanCallbacks]()
		{
			return BooleanCallbacks.GetValue() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		};

		auto IsEnabled = [BooleanCallbacks]()
		{
			return !BooleanCallbacks.IsEnabled.IsSet() || BooleanCallbacks.IsEnabled();
		};

		auto GetToolTipText = [ToolTipText]()
		{
			return ToolTipText;
		};

		return AddWidget( InName,
			MoveTemp(WidgetCallbacks),
			SNew(SCheckBox)
			.OnCheckStateChanged_Lambda(SetCheckState)
			.IsChecked_Lambda(GetCheckState)
			.IsEnabled_Lambda(IsEnabled)
			.ToolTipText_Lambda(GetToolTipText)
		);
	}

	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddString( FText InName, FStringCallbacks&& StringCallbacks, FText ToolTipText, FText HintText)
	{
		check(StringCallbacks.GetValue);
		check(StringCallbacks.SetValue);

		FCallbacks WidgetCallbacks
		{
			.IsVisible = StringCallbacks.IsVisible,
			.Validation = StringCallbacks.Validation,
		};
		if (StringCallbacks.GetDefaultValue != nullptr)
		{
			WidgetCallbacks.IsDefault = [StringCallbacks]()
			{
				return StringCallbacks.GetValue() == StringCallbacks.GetDefaultValue();
			};

			WidgetCallbacks.SetToDefault = [StringCallbacks]()
			{
				StringCallbacks.SetValue(StringCallbacks.GetDefaultValue());
				// nb. OnPropertyChanged is called via SCustomLaunchWidgetTableRow's set to default
			};
		};

		auto SetString = [this, StringCallbacks]( const FText& InText, ETextCommit::Type InTextCommit )
		{
			StringCallbacks.SetValue( InText.ToString() );
			OnPropertyChanged();
		};

		auto GetString = [StringCallbacks]()
		{
			return FText::FromString(StringCallbacks.GetValue());
		};

		auto IsReadOnly = [StringCallbacks]()
		{
			return StringCallbacks.IsEnabled && !StringCallbacks.IsEnabled();
		};

		auto GetHintText = [HintText]()
		{
			return HintText;
		};

		auto GetToolTipText = [ToolTipText]()
		{
			return ToolTipText;
		};

		return AddWidget( InName,
			MoveTemp(WidgetCallbacks),
			SNew(SMultiLineEditableTextBox)
			.AllowMultiLine(false)
			.AutoWrapText(true)
			.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			.Text_Lambda( GetString )
			.OnTextCommitted_Lambda( SetString )
			.IsReadOnly_Lambda( IsReadOnly )
			.HintText_Lambda(GetHintText)
			.ToolTipText_Lambda(GetToolTipText)
		);

	}

	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddDirectoryString( FText InName, FStringCallbacks&& StringCallbacks, bool InMakeRelative, FText TooltipText )
	{
		check(StringCallbacks.GetValue);
		check(StringCallbacks.SetValue);

		FCallbacks WidgetCallbacks
		{
			.IsVisible = StringCallbacks.IsVisible,
			.Validation = StringCallbacks.Validation,
		};
		if (StringCallbacks.GetDefaultValue != nullptr)
		{
			WidgetCallbacks.IsDefault = [StringCallbacks]()
			{
				return StringCallbacks.GetValue() == StringCallbacks.GetDefaultValue();
			};

			WidgetCallbacks.SetToDefault = [StringCallbacks]()
			{
				StringCallbacks.SetValue(StringCallbacks.GetDefaultValue());
				// nb. OnPropertyChanged is called via SCustomLaunchWidgetTableRow's set to default
			};
		};

		auto GetTooltipText = [TooltipText]()
		{
			return TooltipText;
		};

		auto SetString = [this, StringCallbacks]( const FText& InText, ETextCommit::Type InTextCommit )
		{
			StringCallbacks.SetValue( InText.ToString() );
			OnPropertyChanged();
		};

		auto GetString = [StringCallbacks]()
		{
			return FText::FromString(StringCallbacks.GetValue());
		};

		auto OnBrowse = [this,StringCallbacks, InMakeRelative]()
		{
			FString InitialDirectory = StringCallbacks.GetValue();
			if (InitialDirectory.IsEmpty())
			{
				InitialDirectory = this->TreeData->Profile->GetProjectBasePath();
			}
			if (!InitialDirectory.IsEmpty() && FPaths::IsRelative(InitialDirectory))
			{
				InitialDirectory = FPaths::Combine( FPaths::RootDir(), InitialDirectory );
			}

			const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
			FString OutDirectory;
			if (FDesktopPlatformModule::Get()->OpenDirectoryDialog(ParentWindowHandle, LOCTEXT("FolderDialogTitle", "Choose a directory").ToString(), InitialDirectory, OutDirectory))
			{
				if (FPaths::IsUnderDirectory(OutDirectory, FPaths::RootDir()) && InMakeRelative)
				{
					FPaths::MakePathRelativeTo(OutDirectory, *FPaths::RootDir());
				}

				StringCallbacks.SetValue( OutDirectory );
				OnPropertyChanged();
			}

			return FReply::Handled();
		};

		auto IsEnabled = [StringCallbacks]()
		{
			return !StringCallbacks.IsEnabled.IsSet() || StringCallbacks.IsEnabled();
		};

		return AddWidget( InName,
			MoveTemp(WidgetCallbacks),
			SNew(SHorizontalBox)

			// path field
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				.Text_Lambda( GetString )
				.OnTextCommitted_Lambda( SetString )
				.IsEnabled_Lambda( IsEnabled )
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				.ToolTipText_Lambda(GetTooltipText)
			]

			// browse button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4, 0, 0, 0))
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FProjectLauncherStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("DirBrowseTip", "Browse for a folder"))
				.OnClicked_Lambda(OnBrowse)
				.IsEnabled_Lambda(IsEnabled)
				.ContentPadding(2)
				[
					SNew(SImage)
					.Image(FProjectLauncherStyle::Get().GetBrush("PathPickerButton"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		);
	}


	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddFileString(FText InName, FStringCallbacks&& StringCallbacks, FString FileFilter)
	{
		// Invokes file picker dialog and passes the selected
		// file as a string to given callback.

		check(StringCallbacks.GetValue);
		check(StringCallbacks.SetValue);

		FCallbacks WidgetCallbacks
		{
			.IsVisible = StringCallbacks.IsVisible,
		};

		if (StringCallbacks.GetDefaultValue != nullptr)
		{
			WidgetCallbacks.IsDefault = [StringCallbacks]()
			{
				return StringCallbacks.GetValue() == StringCallbacks.GetDefaultValue();
			};

			WidgetCallbacks.SetToDefault = [StringCallbacks]()
			{
				StringCallbacks.SetValue(StringCallbacks.GetDefaultValue());
				// nb. OnPropertyChanged is called via SCustomLaunchWidgetTableRow's set to default
			};
		};

		auto SetString = [this, StringCallbacks]( const FText& InText, ETextCommit::Type InTextCommit )
		{
			StringCallbacks.SetValue( InText.ToString() );
			TreeData->TreeBuilder->OnPropertyChanged();
		};

		auto GetString = [StringCallbacks]()
		{
			return FText::FromString(StringCallbacks.GetValue());
		};

		auto OnBrowse = [this,StringCallbacks, FileFilter]()
		{
			FString InitialDirectory = StringCallbacks.GetValue();
			if (InitialDirectory.IsEmpty())
			{
				InitialDirectory = this->TreeData->Profile->GetProjectBasePath();
			}
			if (!InitialDirectory.IsEmpty() && FPaths::IsRelative(InitialDirectory))
			{
				InitialDirectory = FPaths::Combine( FPaths::RootDir(), InitialDirectory );
			}

			const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
			TArray<FString> OutFiles;
			FString OutDirectory;

			if (FDesktopPlatformModule::Get()->OpenFileDialog(ParentWindowHandle, LOCTEXT("FileDialogTitle", "Choose a file").ToString(), InitialDirectory, "", FileFilter, EFileDialogFlags::None, OutFiles))
			{
				if (OutFiles.Num() > 0)
				{
					FString& FilePath = OutFiles[0];
					if (FPaths::IsUnderDirectory(FilePath, FPaths::RootDir()))
					{
						FPaths::MakePathRelativeTo(FilePath, *FPaths::RootDir());
					}

					StringCallbacks.SetValue(FilePath);
					TreeData->TreeBuilder->OnPropertyChanged();
				}
			}

			return FReply::Handled();
		};

		auto IsEnabled = [StringCallbacks]()
		{
			return !StringCallbacks.IsEnabled.IsSet() || StringCallbacks.IsEnabled();
		};

		return AddWidget( InName,
			MoveTemp(WidgetCallbacks),
			SNew(SHorizontalBox)

			// path field
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(SEditableTextBox)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				.Text_Lambda( GetString )
				.OnTextCommitted_Lambda( SetString )
				.IsEnabled_Lambda( IsEnabled )
				.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
			]

			// browse button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4, 0, 0, 0))
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ButtonStyle(FProjectLauncherStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("FileBrowseTip", "Browse for a file"))
				.OnClicked_Lambda(OnBrowse)
				.IsEnabled_Lambda(IsEnabled)
				.ContentPadding(2)
				[
					SNew(SImage)
					.Image(FProjectLauncherStyle::Get().GetBrush("PathPickerButton"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		);
	}

	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddCommandLineString(FText InName, FStringCallbacks&& StringCallbacks, const TCHAR* InType)
	{
		if (!TreeData->Model->AreExtensionsEnabled())
		{
			return AddString(InName, MoveTemp(StringCallbacks));
		}

		if (InType == nullptr)
		{
			InType = ProjectLauncher::CmdLineType::Launch;
		}

		static std::atomic<uint64> UniqueInstanceID = 0;
		uint64 InstanceID = UniqueInstanceID.fetch_add(1, std::memory_order_relaxed) + 1;;

		check(StringCallbacks.GetValue);
		check(StringCallbacks.SetValue);
		TreeData->OnCommandLineFieldCreated( *this, StringCallbacks, InType );

		FCallbacks WidgetCallbacks
		{
			.IsVisible = StringCallbacks.IsVisible,
			.Validation = StringCallbacks.Validation,
		};
		if (StringCallbacks.GetDefaultValue != nullptr)
		{
			WidgetCallbacks.IsDefault = [StringCallbacks]()
			{
				return StringCallbacks.GetValue() == StringCallbacks.GetDefaultValue();
			};

			WidgetCallbacks.SetToDefault = [StringCallbacks]()
			{
				StringCallbacks.SetValue(StringCallbacks.GetDefaultValue());
				// nb. OnPropertyChanged is called via SCustomLaunchWidgetTableRow's set to default
			};
		};

		auto SetString = [this, StringCallbacks]( const FText& InText, ETextCommit::Type InTextCommit )
		{
			StringCallbacks.SetValue( InText.ToString() );
			OnPropertyChanged();
		};

		auto GetString = [StringCallbacks]()
		{
			return FText::FromString(StringCallbacks.GetValue());
		};

		auto OnGetCmdlineParameterMenuContent = [this, StringCallbacks, InstanceID, TypeName = FString(InType)]()
		{
			const bool bShouldCloseWindowAfterMenuSelection = false;
			const bool bCloseSelfOnly = false;
			const bool bSearchable = false;
			FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr, nullptr, bCloseSelfOnly, &FCoreStyle::Get(), bSearchable );

			for (TSharedPtr<FLaunchExtensionInstance> ExtensionInstance : TreeData->ExtensionInstances)
			{
				ProjectLauncher::ICmdLineParametersExtensionFactory* CmdLineParametersFactory = ExtensionInstance->AsCmdLineParametersFactory();
				if (CmdLineParametersFactory == nullptr || CmdLineParametersFactory->GetCmdLineType() != TypeName)
				{
					continue;
				}

				auto MakeCommandLineSubmenu = [this, StringCallbacks, InstanceID, TypeName, ExtensionInstance]( FMenuBuilder& MenuBuilder )
				{
					ICmdLineParametersExtensionFactory::MakeCmdLineParametersSubmenu(MenuBuilder, InstanceID, TypeName, GetTreeData()->Model, ExtensionInstance.ToSharedRef(), StringCallbacks.SetValue, StringCallbacks.GetValue);
				};

#if 1
				// flat list of extensions - easier to see but risks getting cluttered later on
				MenuBuilder.BeginSection(NAME_None, ExtensionInstance->GetExtension()->GetDisplayName());
				MakeCommandLineSubmenu(MenuBuilder);
				MenuBuilder.EndSection();
#else 
				// extensions in submenus - more scalable, but harder to see everything at a glance
				const bool bInOpenSubMenuOnClick = false;
				MenuBuilder.AddSubMenu(
					ExtensionInstance->GetExtension()->GetDisplayName(),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateLambda(MakeCommandLineSubmenu),
					bInOpenSubMenuOnClick,
					FSlateIcon(),
					bShouldCloseWindowAfterMenuSelection
				);
#endif
			}

			return MenuBuilder.MakeWidget();
		};

		auto GetExtensionButtonVisibility = [this, TypeName = FString(InType)]()
		{
			// see if there's an extension that can support our command line type
			for (TSharedPtr<FLaunchExtensionInstance> ExtensionInstance : TreeData->ExtensionInstances)
			{
				ProjectLauncher::ICmdLineParametersExtensionFactory* CmdLineParametersFactory = ExtensionInstance->AsCmdLineParametersFactory();
				if (CmdLineParametersFactory != nullptr && CmdLineParametersFactory->GetCmdLineType() == TypeName)
				{
					return EVisibility::Visible;
				}
			}

			return EVisibility::Collapsed;
		};

		auto IsEnabled = [StringCallbacks]()
		{
			return !StringCallbacks.IsEnabled.IsSet() || StringCallbacks.IsEnabled();
		};

		return AddWidget( InName,
			MoveTemp(WidgetCallbacks),
			SNew(SHorizontalBox)

			// path field
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			[
				SNew(SMultiLineEditableTextBox)
				.AllowMultiLine(false)
				.AutoWrapText(true)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
				.Text_Lambda( GetString )
				.OnTextCommitted_Lambda( SetString )
				.IsEnabled_Lambda(IsEnabled)
			]

			// command line parameters button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(4, 0, 0, 0))
			.VAlign(VAlign_Center)
			[
				SNew(SComboButton)
				.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
				.ToolTipText(LOCTEXT("CmdLineOptionsLabel", "Add a special parameter to command line"))
				.OnGetMenuContent_Lambda(OnGetCmdlineParameterMenuContent)
				.Visibility_Lambda(GetExtensionButtonVisibility)
				.MenuPlacement(MenuPlacement_BelowRightAnchor)
				.IsEnabled_Lambda(IsEnabled)
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.AddCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		);
	}










	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddInteger( FText InName, TTypeCallbacks<int32>&& TypeCallbacks, FInt32Interval Range, bool bShowSpinner )
	{
		return AddValue<int32>(InName, MoveTemp(TypeCallbacks), Range, bShowSpinner);
	}


	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddFloat( FText InName, TTypeCallbacks<float>&& TypeCallbacks, FFloatInterval Range, bool bShowSpinner )
	{
		return AddValue<float>(InName, MoveTemp(TypeCallbacks), Range, bShowSpinner);
	}


}

#undef LOCTEXT_NAMESPACE
