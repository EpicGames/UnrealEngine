// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProjectLauncherModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Dialog/DialogCommands.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SProjectLauncher.h"
#include "Styling/ProjectLauncherStyle.h"
#include "ILauncherServicesModule.h"
#include "ITargetDeviceServicesModule.h"
#include "ProfileTree/ILaunchProfileTreeBuilder.h"
#include "ProfileTree/BasicProfileTreeBuilder.h"
#include "ProfileTree/CustomProfileTreeBuilder.h"
#include "Extension/BuildCookRunLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"

#define LOCTEXT_NAMESPACE "FProjectLauncherModule"

static const FName TabName("ProjectLauncher");

namespace ProjectLauncher
{
	extern void RegisterTreeBuilderFactory( TSharedRef<ILaunchProfileTreeBuilderFactory> TreeBuilderFactory );
	extern void UnregisterTreeBuilderFactory( TSharedRef<ILaunchProfileTreeBuilderFactory> TreeBuilderFactory );

	extern void RegisterExtension( TSharedRef<FLaunchExtension> Extension );
	extern void UnregisterExtension( TSharedRef<FLaunchExtension> Extension );

	extern void ShutdownLaunchExtensions();

}


class FProjectLauncherModule : public IProjectLauncherModule
{
public:

	virtual void StartupModule() override
	{
		// register default tree builders
		RegisterTreeBuilder( MakeShared<ProjectLauncher::FBasicProfileTreeBuilderFactory>() );
		RegisterTreeBuilder( MakeShared<ProjectLauncher::FCustomProfileTreeBuilderFactory>() );

		// register built-in extensions
		RegisterExtension( MakeShared<ProjectLauncher::FBuildCookRunLaunchExtension>() );

		// register custom styles & other UI elements
		FProjectLauncherStyle::Initialize();
		if (!FDialogCommands::IsRegistered())
		{
			FDialogCommands::Register();
		}

		// register main tab
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(TabName, FOnSpawnTab::CreateRaw(this, &FProjectLauncherModule::HandleSpawnTab))
			.SetDisplayName(LOCTEXT("SpawnTabLabel", "Project Launcher"))
			.SetTooltipText(LOCTEXT("SpawnTabToolTip", "Configure custom launch profiles for advanced packaging, deploying and launching of your project"))
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Launcher.TabIcon"))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());
	}

	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(TabName);
		FProjectLauncherStyle::Shutdown();
		ProjectLauncher::ShutdownLaunchExtensions();
		Model.Reset();
	}

	virtual void RegisterTreeBuilder( TSharedRef<ProjectLauncher::ILaunchProfileTreeBuilderFactory> TreeBuilderFactory ) override
	{
		ProjectLauncher::RegisterTreeBuilderFactory(TreeBuilderFactory);
	}

	virtual void UnregisterTreeBuilder( TSharedRef<ProjectLauncher::ILaunchProfileTreeBuilderFactory> TreeBuilderFactory ) override
	{
		ProjectLauncher::UnregisterTreeBuilderFactory(TreeBuilderFactory);
	}


	virtual void RegisterExtension( TSharedRef<ProjectLauncher::FLaunchExtension> Extension ) override
	{
		ProjectLauncher::RegisterExtension(Extension);
	}

	virtual void UnregisterExtension( TSharedRef<ProjectLauncher::FLaunchExtension> Extension ) override
	{
		ProjectLauncher::UnregisterExtension(Extension);
	}




private:

	TSharedRef<SDockTab> HandleSpawnTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		ILauncherServicesModule& LauncherServicesModule = FModuleManager::LoadModuleChecked<ILauncherServicesModule>("LauncherServices");
		ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>("TargetDeviceServices");

		if (!Model.IsValid())
		{
			Model = MakeShared<ProjectLauncher::FModel>(TargetDeviceServicesModule.GetDeviceProxyManager(), LauncherServicesModule.CreateLauncher(), LauncherServicesModule.GetProfileManager() );
			Model->Init();
		}
		
		auto OnTabActivated = [this](TSharedRef<SDockTab>, ETabActivationCause Cause)
		{
			if (Cause == ETabActivationCause::UserClickedOnTab)
			{
				ILauncherProfilePtr Profile = Model->GetSelectedProfile();
				if (Profile.IsValid())
				{
					Profile->RefreshCustomWarningsAndErrors();
				}
			}
		};


		return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		.OnTabActivated_Lambda(OnTabActivated)
		[
			SNew(SProjectLauncher, Model.ToSharedRef())
		];
	}

	TSharedPtr<ProjectLauncher::FModel> Model;
};


IMPLEMENT_MODULE(FProjectLauncherModule, ProjectLauncher);

#undef LOCTEXT_NAMESPACE
