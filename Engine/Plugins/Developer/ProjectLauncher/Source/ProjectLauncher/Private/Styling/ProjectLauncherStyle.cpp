// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/ProjectLauncherStyle.h"

#include "Styling/StyleColors.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Brushes/SlateBoxBrush.h"
#include "Brushes/SlateImageBrush.h"
#include "Brushes/SlateNoResource.h"
#include "Misc/Paths.h"
#include "Interfaces/IPluginManager.h"
#include "PlatformInfo.h"

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir StyleSet->RootToContentDir
#define RootToCoreContentDir StyleSet->RootToCoreContentDir

TSharedPtr<FSlateStyleSet> FProjectLauncherStyle::StyleSet = nullptr;
FSegmentedProgressBarStyle FProjectLauncherStyle::ProgressBarStyle;

static const FVector2D Icon12x12(12.0f, 12.0f);
static const FVector2D Icon15x15(15.0f, 15.0f);
static const FVector2D Icon16x16(16.0f, 16.0f);
static const FVector2D Icon24x24(24.0f, 24.0f);
static const FVector2D Icon28x28(28.0f, 28.0f);
static const FVector2D Icon36x36(36.0f, 36.0f);
static const FVector2D Icon64x64(64.0f, 64.0f);
static const FVector2D Icon512x512(512.0f, 512.0f);


void FProjectLauncherStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("ProjectLauncherStyle") );

	StyleSet->SetParentStyleName("CoreStyle");
	StyleSet->SetContentRoot(IPluginManager::Get().FindPlugin("ProjectLauncher")->GetBaseDir() / TEXT("Resources"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	StyleSet->Set("WhiteGroupBorder", new CORE_BOX_BRUSH(TEXT("Common/WhiteGroupBorder"), FMargin(4.0f/16.0f)));

	StyleSet->Set("SidePanelRightClose", new IMAGE_BRUSH_SVG(TEXT("SidePanelRightClose"), Icon16x16));
	StyleSet->Set("Icons.ClearLog", new IMAGE_BRUSH_SVG(TEXT("ClearLog"), Icon16x16));

	StyleSet->Set("BadgeOutlined.Error", new IMAGE_BRUSH_SVG(TEXT("BadgeOutlinedError_15"), Icon15x15));
	StyleSet->Set("BadgeOutlined.AllComplete", new IMAGE_BRUSH_SVG(TEXT("BadgeOutlinedSuccess_15-1"), Icon15x15));
	StyleSet->Set("BadgeOutlined.Success", new IMAGE_BRUSH_SVG(TEXT("BadgeOutlinedSuccess_15"), Icon15x15));
	StyleSet->Set("OuterCircle", new IMAGE_BRUSH_SVG(TEXT("outer_circle_white"), Icon36x36));
	/*StyleSet->Set("OuterCircle.Busy", new IMAGE_BRUSH(TEXT("outer_circle_busy"), Icon36x36));*/ // unused - higher res image has less aliasing when rotating
	StyleSet->Set("OuterCircle.Busy", new IMAGE_BRUSH(TEXT("outer_circle_busy_512px"), Icon512x512));
	StyleSet->Set("FullCircle", new IMAGE_BRUSH_SVG(TEXT("full_circle_white"), Icon36x36));

	FButtonStyle HoverHintOnly = FButtonStyle()
		.SetNormal(FSlateNoResource())
		.SetHovered(CORE_BOX_BRUSH("../Editor/Slate/Common/ButtonHoverHint", FMargin(4.f/16.0f), FLinearColor(1.f,1.f,1.f,0.15f)))
		.SetPressed(CORE_BOX_BRUSH("../Editor/Slate/Common/ButtonHoverHint", FMargin(4.f/16.0f), FLinearColor(1.f,1.f,1.f,0.25f)))
		.SetNormalPadding(FMargin(0.f,0.f,0.f,1.f))
		.SetPressedPadding(FMargin(0.f,1.f,0.f,0.f));
	StyleSet->Set( "HoverHintOnly", HoverHintOnly );

	ProgressBarStyle = FSegmentedProgressBarStyle()
		.SetBusyColor(FLinearColor(FColor(0xFF0070E0)))
		.SetPendingColor(FLinearColor(FColor(0xFF1A1A1A)))
		.SetErrorColor(FLinearColor(FColor(0xFFEF3535)))
		.SetCompleteColor(FLinearColor(FColor(0xFF0070E0)))
		.SetCanceledColor(FLinearColor(FColor(0xFF383838)))
		.SetAllCompleteColor(FLinearColor(FColor(0xFF0070E0)))
		.SetFullCircleBrush(StyleSet->GetBrush("FullCircle"))
		.SetOuterCircleBrush(StyleSet->GetBrush("OuterCircle"))
		.SetOuterCircleBusyBrush(StyleSet->GetBrush("OuterCircle.Busy"))
		.SetBadgeSuccessBrush(StyleSet->GetBrush("BadgeOutlined.Success"))
		.SetBadgeErrorBrush(StyleSet->GetBrush("BadgeOutlined.Error"))
		.SetBadgeAllCompleteBrush(StyleSet->GetBrush("BadgeOutlined.AllComplete"));

	StyleSet->Set("Icons.DiffersFromDefault", new CORE_IMAGE_BRUSH_SVG( "../Editor/Slate/Starship/Common/ResetToDefault", Icon16x16) );
	StyleSet->Set("Icons.DeviceManager", new CORE_IMAGE_BRUSH_SVG( "../Editor/Slate/Starship/Common/DeviceManager", Icon16x16) );
	StyleSet->Set("Icons.Plugins", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/Plugins", Icon16x16));
	StyleSet->Set("Icons.Extensions", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/EditorViewport/menu", Icon16x16));

	StyleSet->Set("Icons.Asset", new CORE_IMAGE_BRUSH( "../Editor/Slate/Icons/doc_16x", Icon16x16 ) );
	StyleSet->Set("Icons.WarningWithColor.Small", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/AlertTriangleSolid", Icon16x16, FStyleColors::Warning));

	StyleSet->Set( "PathPickerButton", new CORE_IMAGE_BRUSH( "Icons/ellipsis_12x", Icon12x12 ) );
	StyleSet->Set( "OverflowButton", new CORE_IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 24.f)));


	StyleSet->Set("Icons.Task.Run",	new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/Timecode", Icon16x16));
	StyleSet->Set("Icons.Task.Cleanup", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/GraphEditors/CleanUp", Icon16x16));
	StyleSet->Set("Icons.Task.Launch",	new CORE_IMAGE_BRUSH_SVG("Starship/Common/ProjectLauncher", Icon16x16));
	StyleSet->Set("Icons.Task.Build", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/Adjust", Icon16x16));
	StyleSet->Set("Icons.Task.Zen", new IMAGE_BRUSH_SVG("Zen_16", Icon16x16));
	StyleSet->Set("Icons.Task.Cook", new CORE_IMAGE_BRUSH_SVG( "../Editor/Slate/Starship/Common/CookContent", Icon16x16 ) );
	StyleSet->Set("Icons.Task.Deploy", new CORE_IMAGE_BRUSH_SVG( "../Editor/Slate/Starship/Common/DeviceManager", Icon16x16 ) );
	StyleSet->Set("Icons.Task.Archive", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/SaveCurrent", Icon16x16));
	StyleSet->Set("Icons.Task.Package", new CORE_IMAGE_BRUSH_SVG("../Editor/Slate/Starship/Common/ProjectPackage", Icon16x16));
	StyleSet->Set("Icons.Task.TestAutomation", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Test", Icon16x16));


	StyleSet->Set("Profile.NoPlatform", new CORE_IMAGE_BRUSH_SVG("Starship/Launcher/PaperAirplane", Icon24x24));
	StyleSet->Set("Profile.NoPlatform.Large", new CORE_IMAGE_BRUSH_SVG("Starship/Launcher/PaperAirplane", Icon64x64));

	const FTextBlockStyle NormalText = FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText");
	FTextBlockStyle BoldText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetHighlightColor(FLinearColor(0.3f, 0.3f, 0.3f))
		.SetFont(DEFAULT_FONT("Bold", 10));

	StyleSet->Set("RichTextBlock.Bold", BoldText);

	FSlateStyleRegistry::RegisterSlateStyle( *StyleSet.Get() );
}

#undef RootToContentDir
#undef RootToCoreContentDir

void FProjectLauncherStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle( *StyleSet.Get() );
		ensure( StyleSet.IsUnique() );
		StyleSet.Reset();
	}
}

const ISlateStyle& FProjectLauncherStyle::Get()
{
	return *(StyleSet.Get());
}

const FName& FProjectLauncherStyle::GetStyleSetName()
{
	return StyleSet->GetStyleSetName();
}



const FSlateBrush* FProjectLauncherStyle::GetBrushForTask(ILauncherTaskPtr Task)
{
	if (Task.IsValid())
	{
		FString TaskName = Task->GetName();
		if (TaskName.Contains(TEXT("Cooking in the editor")))
		{
			return GetBrush("Icons.Task.Run");
		}
		else if (TaskName.Contains(TEXT("Post Launch")))
		{
			return GetBrush("Icons.Task.Cleanup");
		}
		else if (TaskName.Contains(TEXT("Launch")))
		{
			return GetBrush("Icons.Task.Launch");
		}
		else if (TaskName.Contains(TEXT("Build")))
		{
			return GetBrush("Icons.Task.Build");
		}
		else if (TaskName.Contains(TEXT("Snapshot")))
		{
			return GetBrush("Icons.Task.Zen");
		}
		else if (TaskName.Contains(TEXT("Cook")))
		{
			return GetBrush("Icons.Task.Cook");
		}
		else if (TaskName.Contains(TEXT("Deploy")))
		{
			return GetBrush("Icons.Task.Deploy");
		}
		else if (TaskName.Contains(TEXT("Archive")))
		{
			return GetBrush("Icons.Task.Archive");
		}
		else if (TaskName.Contains(TEXT("Run")))
		{
			return GetBrush("Icons.Task.Run");
		}
		else if (TaskName.Contains(TEXT("Automated Testing")))
		{
			return GetBrush("Icons.Task.TestAutomation");
		}
	}

	return GetBrush("Icons.Task.Package");
}


const FSlateBrush* FProjectLauncherStyle::GetProfileBrushForPlatform(const PlatformInfo::FTargetPlatformInfo* PlatformInfo, EPlatformIconSize IconSize)
{
	if (PlatformInfo != nullptr)
	{
		return FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(IconSize));
	}

	if (IconSize == EPlatformIconSize::Normal)
	{	
		return GetBrush("Profile.NoPlatform");
	}
	else
	{
		return GetBrush("Profile.NoPlatform.Large");
	}
}

const FSlateBrush* FProjectLauncherStyle::GetProfileBrushForPlatforms(TArray<const PlatformInfo::FTargetPlatformInfo*> PlatformInfos, EPlatformIconSize IconSize)
{
	if (PlatformInfos.Num() == 0)
	{
		return GetProfileBrushForPlatform(nullptr, IconSize);
	}
	else
	{
		// @todo: should we have a dedicated icon for multi-platform?
		return GetProfileBrushForPlatform(PlatformInfos[0], IconSize);
	}
}
