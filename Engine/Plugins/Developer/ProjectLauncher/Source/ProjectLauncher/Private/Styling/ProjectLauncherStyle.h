// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "ILauncherTask.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "SSegmentedProgressBar.h"

namespace PlatformInfo
{
	struct FTargetPlatformInfo;
};

class FProjectLauncherStyle
{
public:
	static void Initialize();
	static void Shutdown();

	static const ISlateStyle& Get();
	static const FName& GetStyleSetName();
	static const FSlateBrush* GetBrush(FName PropertyName, const ANSICHAR* Specifier = nullptr)
	{
		return StyleSet->GetBrush(PropertyName, Specifier);
	}

	static const FSlateBrush* GetBrushForTask(ILauncherTaskPtr Task);
	static const FSlateBrush* GetProfileBrushForPlatform(const PlatformInfo::FTargetPlatformInfo* PlatformInfo, EPlatformIconSize IconSize);
	static const FSlateBrush* GetProfileBrushForPlatforms(TArray<const PlatformInfo::FTargetPlatformInfo*> PlatformInfos, EPlatformIconSize IconSize);

	static const FSegmentedProgressBarStyle& GetProgressBarStyle()
	{
		return ProgressBarStyle;
	}

private:
	static TSharedPtr<FSlateStyleSet> StyleSet;	
	static FSegmentedProgressBarStyle ProgressBarStyle;
};
