// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/LauncherCmdLineUtils.h"
#include "Misc/Parse.h"


namespace ProjectLauncher::CmdLineUtils
{


	bool IsParameterGroup( const FString& InCmdLine, const FString& InParameter )
	{
		// see if InParameter contains just a simple -Key=Value or -Param
		FString ParsedParameter = InParameter;
		FString ParamKey, ParamValue;
		if (InParameter.Split(TEXT("="), &ParamKey, &ParamValue))
		{
			ParamKey += TEXT("=");
			if (FParse::Value(*InParameter, *ParamKey, ParamValue, false))
			{
				ParsedParameter = ParamKey + ParamValue;
			}
		}
		
		return (ParsedParameter.TrimStartAndEnd() != InParameter.TrimStartAndEnd());
	}


	bool TryRemoveParameterGroup( FString& InOutCmdLine, const FString& InParameter )
	{
		if (!IsParameterGroup(InOutCmdLine, InParameter))
		{
			return false;
		}

		// it may contain a group - handle them separately
		const TCHAR* Ptr = InParameter.GetCharArray().GetData();
		FString SubParameter = FParse::Token(Ptr, false);
		while (!SubParameter.IsEmpty())
		{
			RemoveParameter(InOutCmdLine, SubParameter);
			SubParameter = FParse::Token(Ptr, false);
		}

		return true;
	}


	FString GetParameterValue( const FString& InCmdLine, const FString& InParameter )
	{
		FString CommandLine = InCmdLine;

		// InParameter is -Key=
		FString ParamValue;
		if (FParse::Value(*CommandLine, *InParameter, ParamValue, false))
		{
			return ParamValue;
		}

		// InParameter is -Key=Value
		FString ParamKey;
		if (InParameter.Split(TEXT("="), &ParamKey, &ParamValue))
		{
			ParamKey += TEXT("=");
			if (FParse::Value(*CommandLine, *ParamKey, ParamValue, false))
			{
				return ParamValue;
			}
		}

		return FString();
	}


	bool UpdateParameterValue( FString& InOutCmdLine, const FString& InParameter, const FString& NewValue )
	{
		FString ParamKey, ParamValue;
		if (InParameter.Split(TEXT("="), &ParamKey, &ParamValue))
		{
			RemoveParameter(InOutCmdLine, InParameter);

			FString NewParameter = ParamKey + "=" + NewValue;
			AddParameter(InOutCmdLine, NewParameter);
			return true;
		}

		return false;

	}


	FString GetFinalParameter( const FString& InCmdLine, const FString& InParameter )
	{
		FString CommandLine = InCmdLine;

		// get the parameter's key & value
		FString ParamName, ParamValue;
		bool bParameterHasValue = InParameter.Split(TEXT("="), &ParamName, &ParamValue);
		if (!bParameterHasValue)
		{
			ParamName = InParameter;
		}

		// the parameter's value may have been added or alterered - return how it is now
		FString ParamKey = ParamName + TEXT("=");
		if (FParse::Value(*CommandLine, *ParamKey, ParamValue, false))
		{
			return ParamKey + ParamValue;
		}
		
		// the parameter may not have a value or it has been removed - return how it is now
		FString Param = ParamName;
		if (Param.RemoveFromStart(TEXT("-")) && FParse::Param(*CommandLine, *Param))
		{
			return ParamName;
		}

		// return the parameter as-is
		return InParameter;
	}


	void AddParameter( FString& InOutCmdLine, const FString& InParameter )
	{
		FString CommandLine = InOutCmdLine + TEXT(" ") + InParameter;
		InOutCmdLine = CommandLine;
	}


	bool RemoveParameter( FString& InOutCmdLine, const FString& InParameter )
	{
		if (TryRemoveParameterGroup(InOutCmdLine, InParameter))
		{
			return true;
		}

		FString Parameter = GetFinalParameter(InOutCmdLine, InParameter);
		FString CommandLine = InOutCmdLine;

		// first try to remove the parameter and the preceding space
		FString ParameterWithSpace = TEXT(" ") + Parameter;
		if (CommandLine.ReplaceInline(*ParameterWithSpace, TEXT(""), ESearchCase::IgnoreCase) > 0)
		{
			InOutCmdLine = CommandLine;
			return true;
		}

		// next try to remove the parameter with a trailing space
		ParameterWithSpace = Parameter + TEXT(" ");
		if (CommandLine.ReplaceInline(*ParameterWithSpace, TEXT(""), ESearchCase::IgnoreCase) > 0)
		{
			InOutCmdLine = CommandLine;
			return true;
		}

		// a little unexpected... just remove the parameter on its own
		if (CommandLine.ReplaceInline(*Parameter, TEXT(""), ESearchCase::IgnoreCase) > 0)
		{
			InOutCmdLine = CommandLine;
			return true;
		}

		return false;
	}


	bool IsParameterUsed( const FString& InCmdLine, const FString& InParameter )
	{
		FString Parameter = GetFinalParameter(InCmdLine, InParameter);
		return InCmdLine.Contains(Parameter, ESearchCase::IgnoreCase);
	}


	void SetParameterUsed( FString& InOutCmdLine, const FString& InParameter, bool bUsed )
	{
		bool bIsUsed = IsParameterUsed(InOutCmdLine, InParameter);
		if (bIsUsed != bUsed)
		{
			if (bIsUsed)
			{
				RemoveParameter(InOutCmdLine, InParameter);
			}
			else
			{
				AddParameter(InOutCmdLine, InParameter);
			}
		}
	}
}

