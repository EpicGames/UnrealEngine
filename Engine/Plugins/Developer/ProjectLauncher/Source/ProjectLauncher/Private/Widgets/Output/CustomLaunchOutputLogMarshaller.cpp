// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Output/CustomLaunchOutputLogMarshaller.h"
#include "Model/ProjectLauncherModel.h"
#include "Math/ColorList.h"
#include "Styling/AppStyle.h"
#include "Framework/Text/SlateTextRun.h"
#include "Framework/Text/ISlateLineHighlighter.h"


namespace ProjectLauncher
{
	FLaunchLogTextLayoutMarshaller::FLaunchLogTextLayoutMarshaller(const TSharedRef<ProjectLauncher::FModel>& InModel) 
		: Model(InModel)
		, TextLayout(nullptr)
	{
		MessageStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("MonospacedText");

		static const FSlateColor DisplayColors[] =
		{
			FSlateColor(FColorList::White),
			FSlateColor(FColorList::Green),
			FSlateColor(FColorList::Magenta),
			FSlateColor(FColorList::Cyan),
			FSlateColor(FColorList::LimeGreen),
			FSlateColor(FColorList::NeonPink),
			FSlateColor(FColorList::NeonBlue),
		};

		for (const FSlateColor& DisplayColor : DisplayColors)
		{
			FTextBlockStyle& DisplayStyle = DisplayStyles.Add_GetRef(MessageStyle);
			DisplayStyle.ColorAndOpacity = DisplayColor;
		}

		WarningStyle = MessageStyle;
		WarningStyle.ColorAndOpacity = FSlateColor(FColor::Yellow);

		ErrorStyle = MessageStyle;
		ErrorStyle.ColorAndOpacity = FSlateColor(FColor::Red);
	}


	void FLaunchLogTextLayoutMarshaller::SetText(const FString& SourceString, FTextLayout& TargetTextLayout)
	{
		DisplayStyleIndex = 0;
		TextLayout = &TargetTextLayout;
		FlushPendingLogMessages();
	}


	void FLaunchLogTextLayoutMarshaller::GetText(FString& TargetString, const FTextLayout& SourceTextLayout)
	{
		SourceTextLayout.GetAsText(TargetString);
	}


	void FLaunchLogTextLayoutMarshaller::MakeDirty()
	{
		FBaseTextLayoutMarshaller::MakeDirty();
		NumFilteredMessages = 0;
	}


	void FLaunchLogTextLayoutMarshaller::AddPendingLogMessage( TSharedPtr<FLaunchLogMessage> Message )
	{
		PendingMessages.Enqueue(Message);
	}


	void FLaunchLogTextLayoutMarshaller::RefreshAllLogMessages()
	{
		MakeDirty();
		DisplayStyleIndex = 0;

		for (TSharedPtr<FLaunchLogMessage> Message : Model->LaunchLogMessages)
		{
			PendingMessages.Enqueue(Message);
		}
	}


	void FLaunchLogTextLayoutMarshaller::SetFilter( ELogFilter InLogFilter )
	{
		LogFilter = InLogFilter;
		RefreshAllLogMessages();
	}


	void FLaunchLogTextLayoutMarshaller::SetFilterString( const FString& InLogFilterString )
	{
		LogFilterString = InLogFilterString;
		RefreshAllLogMessages();
	}


	bool FLaunchLogTextLayoutMarshaller::FlushPendingLogMessages()
	{
		bool bRefreshLog = false;

		TArray<FTextLayout::FNewLineData> LinesToAdd;

		TSharedPtr<FLaunchLogMessage> Message;
		while (PendingMessages.Dequeue(Message))
		{
			if (LogFilter == ELogFilter::Errors && Message->Verbosity != ELogVerbosity::Error && Message->Verbosity != ELogVerbosity::Fatal)
			{
				continue;
			}
			else if (LogFilter == ELogFilter::WarningsAndErrors && Message->Verbosity != ELogVerbosity::Warning && Message->Verbosity != ELogVerbosity::Error && Message->Verbosity != ELogVerbosity::Fatal)
			{
				continue;
			}
			else if (!LogFilterString.IsEmpty() && !Message->Message->Contains(LogFilterString))
			{
				continue;
			}

			NumFilteredMessages++;
			LinesToAdd.Emplace(MoveTemp(Message->Message), GetRun(Message));

			bRefreshLog = true;
		}

		if (bRefreshLog)
		{
			TextLayout->AddLines(LinesToAdd);
		}

		return bRefreshLog;
	}


	TArray<TSharedRef<IRun>> FLaunchLogTextLayoutMarshaller::GetRun(TSharedPtr<FLaunchLogMessage> Message)
	{
		TArray<TSharedRef<IRun>> Runs;

		if (Message->Verbosity == ELogVerbosity::Display)
		{
			// special case for "Display" log verbosity (aka the RunUAT command line parameter)
			// draw each separate UAT command in a different colour on the same line
			const TCHAR* CommandLinePtr = Message->Message->GetCharArray().GetData();

			FTextRange Range(0,0); 
			while(ParseUATCommand(CommandLinePtr, Range))
			{
				Runs.Add(FSlateTextRun::Create(FRunInfo(), Message->Message, DisplayStyles[DisplayStyleIndex], Range));
				DisplayStyleIndex = (DisplayStyleIndex+1) % DisplayStyles.Num();
			}
		}
		else
		{
			const FTextBlockStyle* TextStyle = &MessageStyle;

			if (Message->Verbosity == ELogVerbosity::Warning)
			{
				TextStyle = &WarningStyle;
			}
			else if (Message->Verbosity == ELogVerbosity::Error || Message->Verbosity == ELogVerbosity::Fatal)
			{
				TextStyle = &ErrorStyle;
			}

			Runs.Add(FSlateTextRun::Create(FRunInfo(), Message->Message, *TextStyle));
		}

		return MoveTemp(Runs);
	}
	
	bool FLaunchLogTextLayoutMarshaller::ParseUATCommand( const TCHAR*& CommandLinePtr, FTextRange& TextRange ) const
	{
		const TCHAR* StartCommandLinePtr = CommandLinePtr;

		const TCHAR* KnownPrefix = TEXT("Parsing command line: ");
		const int32 KnownPrefixLength = FCString::Strlen(KnownPrefix);
		if (FCString::Strnicmp(CommandLinePtr, KnownPrefix, KnownPrefixLength) == 0)
		{
			CommandLinePtr += KnownPrefixLength;
		}

		const TCHAR* PrevCommandLinePtr = nullptr;
		FString Token;
		while (CommandLinePtr != nullptr && *CommandLinePtr != TEXT('\0') && FParse::Token(CommandLinePtr, Token, false))
		{
			if (PrevCommandLinePtr != nullptr && !Token.StartsWith(TEXT("-")))
			{
				CommandLinePtr = PrevCommandLinePtr;
				break;
			}

			PrevCommandLinePtr = CommandLinePtr;
		}

		TextRange.BeginIndex = TextRange.EndIndex;
		TextRange.EndIndex = TextRange.BeginIndex + (CommandLinePtr - StartCommandLinePtr);
		return TextRange.EndIndex > TextRange.BeginIndex;
	}
}
