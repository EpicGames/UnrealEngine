// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/SpscQueue.h"
#include "Templates/SharedPointer.h"
#include "Styling/SlateTypes.h"
#include "Framework/Text/TextLayout.h"
#include "Framework/Text/BaseTextLayoutMarshaller.h"


namespace ProjectLauncher
{
	class FModel;
	struct FLaunchLogMessage;

	enum class ELogFilter : uint8
	{
		All,
		WarningsAndErrors,
		Errors,
	};


	// this class is responsible for marshalling the output log from FModel into an associated multiline text box
	class FLaunchLogTextLayoutMarshaller : public FBaseTextLayoutMarshaller
	{
	public:
		FLaunchLogTextLayoutMarshaller(const TSharedRef<ProjectLauncher::FModel>& InModel);
		virtual void SetText(const FString& SourceString, FTextLayout& TargetTextLayout) override;
		virtual void GetText(FString& TargetString, const FTextLayout& SourceTextLayout) override;
		virtual void MakeDirty() override;

		ELogFilter GetFilter() const { return LogFilter; }
		void SetFilter( ELogFilter InLogFilter );

		FString GetFilterString() const { return LogFilterString; }
		void SetFilterString( const FString& InLogFilterString );

		void AddPendingLogMessage( TSharedPtr<FLaunchLogMessage> Message );
		bool FlushPendingLogMessages();

		void RefreshAllLogMessages();
		int32 GetNumFilteredMessages() const { return NumFilteredMessages; }
	protected:
		TArray<TSharedRef<class IRun>> GetRun(TSharedPtr<FLaunchLogMessage> Message);
		bool ParseUATCommand( const TCHAR*& CommandLinePtr, FTextRange& TextRange ) const;

		FTextBlockStyle MessageStyle;
		FTextBlockStyle WarningStyle;
		FTextBlockStyle ErrorStyle;
		TArray<FTextBlockStyle> DisplayStyles;

		const TSharedRef<ProjectLauncher::FModel> Model;
		TSpscQueue<TSharedPtr<FLaunchLogMessage>> PendingMessages;
		ELogFilter LogFilter = ELogFilter::All;
		FString LogFilterString;
		FTextLayout* TextLayout;
		int32 NumFilteredMessages = 0;
		int32 DisplayStyleIndex = 0;

	};
}
