// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Output/SCustomLaunchLaunchPanel.h"

#include "Styling/AppStyle.h"
#include "Styling/ProjectLauncherStyle.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SProgressBar.h"
#include "Widgets/Output/SCustomLaunchOutputLog.h"
#include "ITargetDeviceProxy.h"
#include "PlatformInfo.h"



#define LOCTEXT_NAMESPACE "SCustomLaunchLaunchPanel"



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchLaunchPanel::Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel)
{
	Model = InModel;
	OnCloseClicked = InArgs._OnCloseClicked;
	OnRerunClicked = InArgs._OnRerunClicked;

	LaunchLogTextMarshaller = MakeShared<ProjectLauncher::FLaunchLogTextLayoutMarshaller>(InModel);
	OutputLog = SNew(SCustomLaunchOutputLog, InModel, LaunchLogTextMarshaller.ToSharedRef());

	ChildSlot
	[
		SNew(SVerticalBox)

		// top banner
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
			.Padding(16)
			[
				SNew(SVerticalBox)
				
				// profile details & control buttons
				+SVerticalBox::Slot()
				.Padding(0)
				[
					SNew(SHorizontalBox)

					// profile icon
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4,0)
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.DesiredSizeOverride(FVector2D(44,44))
						.Image( this, &SCustomLaunchLaunchPanel::GetProfileImage )
					]

					// profile details
					+SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(4)
					.VAlign(VAlign_Center)
					[
						SNew(SVerticalBox)

						// profile name
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(STextBlock)
							.Text( this, &SCustomLaunchLaunchPanel::GetProfileName )
						]

						// profile description
						+SVerticalBox::Slot()
						.AutoHeight()
						.Padding(2)
						[
							SNew(STextBlock)
							.Text( this, &SCustomLaunchLaunchPanel::GetProfileDescription )
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]
					]

					// spacer
					+SHorizontalBox::Slot()
					.FillWidth(1)
					[
						SNew(SSpacer)
					]

					// control buttons
					+SHorizontalBox::Slot()
					.Padding(4,0,0,0)
					.AutoWidth()
					.VAlign(VAlign_Top)
					[
						SNew(SHorizontalBox)

						// cancel button
						+SHorizontalBox::Slot()
						.Padding(FMargin(4,0,0,0))
						[
							SNew(SButton)
							.OnClicked( this, &SCustomLaunchLaunchPanel::OnCancelButtonClicked )
							.Visibility_Lambda( [this]() { return !IsIdle() ? EVisibility::Visible : EVisibility::Collapsed; } )
							.IsEnabled_Lambda( [this]() { return IsBuilding(); } )
							.Text(LOCTEXT("CancelButtonLabel", "Cancel"))
							.ContentPadding(4)
						]

						// retry button
						+SHorizontalBox::Slot()
						.Padding(FMargin(4,0,0,0))
						[
							SNew(SButton)
							.OnClicked( this, &SCustomLaunchLaunchPanel::OnRetryButtonClicked )
							.Visibility_Lambda( [this]() { return IsIdle() ? EVisibility::Visible : EVisibility::Collapsed; } )
							.Text(LOCTEXT("RetryButtonLabel", "Retry"))
							.ContentPadding(4)
						]

						// done button
						+SHorizontalBox::Slot()
						.Padding(FMargin(4,0,0,0))
						[
							SNew(SButton)
							.OnClicked( this, &SCustomLaunchLaunchPanel::OnDoneButtonClicked )
							.Visibility_Lambda( [this]() { return IsIdle() ? EVisibility::Visible : EVisibility::Collapsed; } )
							.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
							.Text(LOCTEXT("DoneButtonLabel", "Done"))
							.ContentPadding(4)
						]
					]
				]

#if 0
				// overall progress bar (only per-task granulatity)
				+SVerticalBox::Slot()
				.Padding(0,16)
				.AutoHeight()
				[
					SNew(SProgressBar)
					.Percent(this, &SCustomLaunchLaunchPanel::GetProgressPercent)
				]
#else
				// experimental subway progress bar
				+SVerticalBox::Slot()
				.Padding(0,16)
				.AutoHeight()
				[
					SAssignNew(SubwayProgressBar, SSegmentedProgressBar)
					.Style(FProjectLauncherStyle::GetProgressBarStyle())
				]
#endif

				// progress description
				+SVerticalBox::Slot()
				.Padding(0)
				.AutoHeight()
				[
					SNew(SGridPanel)
					.FillColumn(2, 1.0f)
					
					// project name
					+SGridPanel::Slot(0,0)
					.Padding(0,2)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0,0,6,0)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ProjectLabel", "Project"))
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text( this, &SCustomLaunchLaunchPanel::GetProfileProjectName )
						]
					]

					// configuration
					+SGridPanel::Slot(1,0)
					.Padding(32,2)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0,0,6,0)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ConfigurationLabel", "Configuration"))
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text( this, &SCustomLaunchLaunchPanel::GetProfileConfigurationName )
						]
					]

					// task progress
					+SGridPanel::Slot(2,0)
					.HAlign(HAlign_Right)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.AutoWidth()
						.Padding(0,0,6,0)
						[
							SNew(STextBlock)
							.Text( this, &SCustomLaunchLaunchPanel::GetCurrentTaskDescription )
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]

						+SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text( this, &SCustomLaunchLaunchPanel::GetProgressDescription )
						]
					]

					// target name
					+SGridPanel::Slot(0,1)
					.Padding(0,2)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0,0,6,0)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("TargetLabel", "Target"))
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text( this, &SCustomLaunchLaunchPanel::GetProfileTargetName )
						]
					]
					
					// content scheme
					+SGridPanel::Slot(1,1)
					.Padding(32,2)
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0,0,6,0)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("ContentLabel", "Content"))
							.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.Text( this, &SCustomLaunchLaunchPanel::GetProfileContentSchemeName )
						]
					]

					// overall build duration
					+SGridPanel::Slot(2,1)
					.HAlign(HAlign_Right)
					[
						SNew(STextBlock)
						.Text( this, &SCustomLaunchLaunchPanel::GetTotalDurationDescription )
					]
				]
			]
		]


		// output log
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,4)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
			[
				SNew(SHorizontalBox)

				// output log title
				+SHorizontalBox::Slot()
				.FillWidth(1)
				.Padding(4,2)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)				
					.Text(LOCTEXT("OutputLogAreaTitle", "Output Log"))
				]

				// filter button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4,2)
				.VAlign(VAlign_Center)
				[
					OutputLog->CreateFilterWidget()
				]
			]
		]

		// main output log
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			OutputLog.ToSharedRef()
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



void SCustomLaunchLaunchPanel::SetLauncherWorker( const ILauncherWorkerRef& Worker )
{
	LauncherWorker = Worker;

	Worker->GetTasks(TaskList);

	if (SubwayProgressBar.IsValid())
	{
		SubwayProgressBar->ClearChildren();
		for (ILauncherTaskPtr Task : TaskList)
		{
			SubwayProgressBar->AddSlot()
			.Image(FProjectLauncherStyle::GetBrushForTask(Task))
			.State(this, &SCustomLaunchLaunchPanel::GetSegmentedProgressBarState, Task)
			.ToolTipText(FText::FromString(Task->GetDesc()))
			;
		}
	}

	OutputLog->RefreshLog();

	Worker->OnOutputReceived().AddRaw(this, &SCustomLaunchLaunchPanel::HandleOutputReceived);
}

void SCustomLaunchLaunchPanel::ClearLog()
{
	Model->ClearLogMessages();
	LaunchLogTextMarshaller->RefreshAllLogMessages();
	OutputLog->RefreshLog();
}


void SCustomLaunchLaunchPanel::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (LaunchLogTextMarshaller->FlushPendingLogMessages())
	{
		OutputLog->RequestForceScroll(true);
	}
}


void SCustomLaunchLaunchPanel::HandleOutputReceived(const FString& InMessage)
{
	ELogVerbosity::Type Verbosity = ELogVerbosity::Log;

	if (InMessage.StartsWith(TEXT("Parsing command line:"), ESearchCase::CaseSensitive))
	{
		Verbosity = ELogVerbosity::Display;
	}
	else if ( InMessage.Contains(TEXT("Fatal error"), ESearchCase::IgnoreCase) )
	{
		Verbosity = ELogVerbosity::Fatal;
	}
	else if ( InMessage.Contains(TEXT("Error:"), ESearchCase::IgnoreCase) || InMessage.StartsWith(TEXT("Result: Failed "), ESearchCase::IgnoreCase ) )
	{
		Verbosity = ELogVerbosity::Error;
	}
	else if ( InMessage.Contains(TEXT("Warning:"), ESearchCase::IgnoreCase) )
	{
		Verbosity = ELogVerbosity::Warning;
	}

	
	
	TSharedPtr<ProjectLauncher::FLaunchLogMessage> Message = Model->AddLogMessage(InMessage, Verbosity);
	LaunchLogTextMarshaller->AddPendingLogMessage(Message);
}


bool SCustomLaunchLaunchPanel::IsBuilding() const
{
	ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();
	if (LauncherWorkerPtr != nullptr)
	{
		return LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Busy;
	}

	return false;
}

bool SCustomLaunchLaunchPanel::IsIdle() const
{
	ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();
	if (LauncherWorkerPtr != nullptr)
	{
		return LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Canceled || LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Completed;
	}

	return false;
}

const FSlateBrush* SCustomLaunchLaunchPanel::GetProfileImage() const
{
	ILauncherProfilePtr LauncherProfile = GetLauncherProfile();
	if (LauncherProfile != nullptr)
	{
		// @todo: should there be a dedicated icon for multi-platform?
		TArray<const PlatformInfo::FTargetPlatformInfo*> PlatformInfos = Model->GetPlatformInfosForAllCommands(LauncherProfile);
		return FProjectLauncherStyle::GetProfileBrushForPlatforms(PlatformInfos, EPlatformIconSize::Large);
	}

	return FStyleDefaults::GetNoBrush();
}


FText SCustomLaunchLaunchPanel::GetProfileName() const
{
	ILauncherProfilePtr LauncherProfile = GetLauncherProfile();
	if (LauncherProfile != nullptr)
	{
		return FText::FromString(LauncherProfile->GetName());
	}

	return FText::GetEmpty();
}


FText SCustomLaunchLaunchPanel::GetProfileDescription() const
{
	ILauncherProfilePtr LauncherProfile = GetLauncherProfile();
	if (LauncherProfile != nullptr)
	{
		if (Model->IsBasicLaunchProfile(LauncherProfile))
		{
			const TSharedPtr<ITargetDeviceProxy> DeviceProxy = Model->GetDeviceProxy(Model->GetBasicLaunchProfile()->GetFirstBuildCookRun().ToSharedRef());
			if (DeviceProxy.IsValid())
			{
				return FText::FromString(DeviceProxy->GetName());
			}
		}
		else
		{
			return FText::FromString(LauncherProfile->GetDescription());
		}
	}

	return FText::GetEmpty();
}


FText SCustomLaunchLaunchPanel::GetProfileProjectName() const
{
	ILauncherProfilePtr LauncherProfile = GetLauncherProfile();
	if (LauncherProfile != nullptr)
	{
		return FText::FromString(LauncherProfile->GetProjectName());
	}

	return FText::GetEmpty();
}
FText SCustomLaunchLaunchPanel::GetProfileConfigurationName() const
{
	ILauncherProfilePtr LauncherProfile = GetLauncherProfile();
	if (LauncherProfile != nullptr && LauncherProfile->GetFirstBuildCookRun().IsValid())
	{
		return EBuildConfigurations::ToText(LauncherProfile->GetFirstBuildCookRun()->GetBuildConfiguration());
	}

	return FText::GetEmpty();
}
FText SCustomLaunchLaunchPanel::GetProfileTargetName() const
{
	ILauncherProfilePtr LauncherProfile = GetLauncherProfile();
	if (LauncherProfile != nullptr && LauncherProfile->GetFirstBuildCookRun().IsValid())
	{
		TArray<FString> BuildTargets = LauncherProfile->GetFirstBuildCookRun()->GetBuildTargets();
		if (BuildTargets.Num() == 0 || BuildTargets.Contains(FString()))
		{
			return LOCTEXT("DefaultTargetName", "Target Default");
		}
		else
		{
			return FText::FromString(FString::Join(BuildTargets, TEXT(", ")));
		}
	}

	return FText::GetEmpty();
}


FText SCustomLaunchLaunchPanel::GetProfileContentSchemeName() const
{
	ILauncherProfilePtr LauncherProfile = GetLauncherProfile();
	if (LauncherProfile != nullptr)
	{
		if (Model->IsAdvancedProfile(LauncherProfile.ToSharedRef()))
		{
			return LOCTEXT("AdvancedProfileInfo", "Advanced/Legacy Profile");
		}
		else if (LauncherProfile->GetBuildCookRunCommands().Num() > 1)
		{
			return LOCTEXT("MultipleContentSchemeLabel", "Multiple"); // @todo: worth trying something fancy here? collecting all content schemes we're using
		}
		else if (LauncherProfile->GetFirstBuildCookRun().IsValid())
		{
			ProjectLauncher::EContentScheme ContentScheme = Model->DetermineProfileContentScheme(LauncherProfile->GetFirstBuildCookRun().ToSharedRef());
			return ProjectLauncher::GetContentSchemeDisplayName(ContentScheme);
		}
		else if (LauncherProfile->GetUATCommands().Num() > 0)
		{
			return LOCTEXT("NoBuildCookRunsLabel", "Custom Profile");
		}

	}

	return FText::GetEmpty();
}


FText SCustomLaunchLaunchPanel::GetCurrentTaskDescription() const
{
	if (TaskList.Num() > 0 && LauncherWorker.IsValid())
	{
		if (TaskList.Last()->GetStatus() == ELauncherTaskStatus::Completed)
		{
			return LOCTEXT("TaskStatusAllCompleted", "Completed All Tasks");
		}

		ILauncherTaskPtr LatestLauncherTask = TaskList[0];
		for (const ILauncherTaskPtr& LauncherTask : TaskList)
		{
			if (LauncherTask->GetStatus() == ELauncherTaskStatus::Canceled || LauncherTask->GetStatus() == ELauncherTaskStatus::Pending)
			{
				break;
			}
			LatestLauncherTask = LauncherTask;
		}

		return FText::FromString(LatestLauncherTask->GetDesc());
	}

	return FText::GetEmpty();
}

FText SCustomLaunchLaunchPanel::GetTotalDurationDescription() const
{
	if (TaskList.Num() > 0 && LauncherWorker.IsValid())
	{
		FTimespan Duration;
		for (const ILauncherTaskPtr& LauncherTask : TaskList)
		{
			Duration += LauncherTask->GetDuration();
		}

		return FText::AsTimespan(Duration);
	}

	return FText::GetEmpty();
}

FText SCustomLaunchLaunchPanel::GetProgressDescription() const
{
	if (TaskList.Num() > 0 && LauncherWorker.IsValid())
	{
		TOptional<float> ProgressPercent = GetProgressPercent();

		for (const ILauncherTaskPtr& LauncherTask : TaskList)
		{
			if (LauncherTask->GetStatus() == ELauncherTaskStatus::Busy && ProgressPercent.IsSet())
			{
				return FText::AsPercent(ProgressPercent.GetValue());
			}
			else if (LauncherTask->GetStatus() == ELauncherTaskStatus::Failed)
			{
				return LOCTEXT("TaskStatusFailed", "Failed");
			}
		}

		switch (TaskList.Last()->GetStatus())
		{
			case ELauncherTaskStatus::Canceled: return LOCTEXT("TaskStatusCanceled", "Canceled");
			//case ELauncherTaskStatus::Completed: return LOCTEXT("TaskStatusCompleted", "Completed");
		}
	}

	return FText::GetEmpty();
}

TOptional<float> SCustomLaunchLaunchPanel::GetProgressPercent() const
{
	if (TaskList.Num() > 0 && LauncherWorker.IsValid())
	{
		int32 NumFinished = 0;
		for (int32 TaskIndex = 0; TaskIndex < TaskList.Num(); ++TaskIndex)
		{
			if (TaskList[TaskIndex]->IsFinished())
			{
				++NumFinished;
			}
		}

		return ((float)NumFinished / TaskList.Num());
	}

	TOptional<float> Result;
	return Result;
}

SSegmentedProgressBar::EState SCustomLaunchLaunchPanel::GetSegmentedProgressBarState( ILauncherTaskPtr Task ) const
{
	if (Task.IsValid())
	{
		switch (Task->GetStatus())
		{
			case ELauncherTaskStatus::Busy:      return SSegmentedProgressBar::EState::Pending;
			case ELauncherTaskStatus::Canceled:  return SSegmentedProgressBar::EState::Canceled;
			case ELauncherTaskStatus::Completed: return SSegmentedProgressBar::EState::Completed;
			case ELauncherTaskStatus::Failed:    return SSegmentedProgressBar::EState::Failed;
			case ELauncherTaskStatus::Pending:   return SSegmentedProgressBar::EState::Pending;
		}
	}

	return SSegmentedProgressBar::EState::None;
}

FReply SCustomLaunchLaunchPanel::OnCancelButtonClicked()
{
	ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();
	if (LauncherWorkerPtr != nullptr && LauncherWorkerPtr->GetStatus() == ELauncherWorkerStatus::Busy)
	{
		LauncherWorkerPtr->Cancel();
	}

	return FReply::Handled();
}


FReply SCustomLaunchLaunchPanel::OnRetryButtonClicked()
{
	check(IsIdle());
	if (OnRerunClicked.IsBound())
	{
		OnRerunClicked.Execute();
	}
	return FReply::Handled();
}


FReply SCustomLaunchLaunchPanel::OnDoneButtonClicked()
{
	check(IsIdle());
	if (OnCloseClicked.IsBound())
	{
		OnCloseClicked.Execute();
	}
	return FReply::Handled();
}

ILauncherProfilePtr SCustomLaunchLaunchPanel::GetLauncherProfile() const
{
	ILauncherWorkerPtr LauncherWorkerPtr = LauncherWorker.Pin();
	return (LauncherWorkerPtr != nullptr) ? LauncherWorkerPtr->GetLauncherProfile() : nullptr;
}



#undef LOCTEXT_NAMESPACE
