// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "SSegmentedProgressBar.h"
#include "Widgets/Output/CustomLaunchOutputLogMarshaller.h"
#include "Model/ProjectLauncherModel.h"
#include "ILauncherWorker.h"

class SWidget;
class SCustomLaunchOutputLog;
class ITableRow;
class STableViewBase;
template<typename T> class SListView;


class SCustomLaunchLaunchPanel
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomLaunchLaunchPanel) {}
		SLATE_EVENT(FOnClicked, OnCloseClicked)
		SLATE_EVENT(FOnClicked, OnRerunClicked)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel);
	void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;


	void SetLauncherWorker( const ILauncherWorkerRef& Worker );
	void ClearLog();

private:
	bool IsBuilding() const;
	bool IsIdle() const;
	const FSlateBrush* GetProfileImage() const;
	FText GetProfileName() const;
	FText GetProfileDescription() const;
	FText GetProfileProjectName() const;
	FText GetProfileConfigurationName() const;
	FText GetProfileTargetName() const;
	FText GetProfileContentSchemeName() const;	
	FText GetCurrentTaskDescription() const;
	FText GetProgressDescription() const;
	FText GetTotalDurationDescription() const;
	TOptional<float> GetProgressPercent() const;
	SSegmentedProgressBar::EState GetSegmentedProgressBarState(ILauncherTaskPtr Task) const;
	FReply OnCancelButtonClicked();
	FReply OnRetryButtonClicked();
	FReply OnDoneButtonClicked();

	ILauncherProfilePtr GetLauncherProfile() const;

	void HandleOutputReceived(const FString& InMessage);

	FOnClicked OnCloseClicked;
	FOnClicked OnRerunClicked;

	TWeakPtr<ILauncherWorker> LauncherWorker;
	TArray<ILauncherTaskPtr> TaskList;

	TSharedPtr<ProjectLauncher::FLaunchLogTextLayoutMarshaller> LaunchLogTextMarshaller;
	TSharedPtr<SCustomLaunchOutputLog> OutputLog;
	TSharedPtr<SSegmentedProgressBar> SubwayProgressBar;

	TSharedPtr<ProjectLauncher::FModel> Model;
};
