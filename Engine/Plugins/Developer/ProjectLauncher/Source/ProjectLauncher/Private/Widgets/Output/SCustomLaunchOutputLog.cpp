// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Output/SCustomLaunchOutputLog.h"

#include "Styling/AppStyle.h"
#include "Styling/ProjectLauncherStyle.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Application/SlateApplication.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Misc/StringBuilder.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "DesktopPlatformModule.h"

#define LOCTEXT_NAMESPACE "SCustomLaunchOutputLog"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchOutputLog::Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel, const TSharedRef<ProjectLauncher::FLaunchLogTextLayoutMarshaller>& InLaunchLogTextMarshaller)
{
	Model = InModel;
	LaunchLogTextMarshaller = InLaunchLogTextMarshaller;

	LogMessageTextBox = SNew(SMultiLineEditableTextBox)
		.Style(FAppStyle::Get(), "Log.TextBox")
		.Marshaller(LaunchLogTextMarshaller)
		.IsReadOnly(true)
		.AlwaysShowScrollbars(true)
		.AutoWrapText_Lambda( [this] () { return bWordWrap;} )
		.OnVScrollBarUserScrolled(this, &SCustomLaunchOutputLog::OnUserScrolled)
		.ContextMenuExtender(this, &SCustomLaunchOutputLog::ExtendTextBoxMenu);
		;

	ChildSlot
	[
		SNew(SVerticalBox)

		+SVerticalBox::Slot()
		.FillHeight(1)
		.Padding(0)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				LogMessageTextBox.ToSharedRef()
			]
		]

		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
			[
				SNew(SHorizontalBox)

				// padding
				+SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SSpacer)
				]

				// copy button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(4,2)
				[
					SAssignNew(CopyButton, SButton)
					.OnClicked(this, &SCustomLaunchOutputLog::OnCopyClicked)
					.IsEnabled_Lambda([InModel]() { return (InModel->GetNumLogMessages() > 0); } )
					.ToolTipText(LOCTEXT("CopyButtonTip", "Copy entire log to the clipboard"))
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("GenericCommands.Copy"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				]

				// clear button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(4,2)
				[
					SAssignNew(ClearButton, SButton)
					.OnClicked(this, &SCustomLaunchOutputLog::OnClearClicked)
					.IsEnabled_Lambda([InModel]() { return (InModel->GetNumLogMessages() > 0); } )
					.ToolTipText(LOCTEXT("ClearButtonTip", "Clear all messages"))
					[
						SNew(SImage)
						.Image(FProjectLauncherStyle::Get().GetBrush("Icons.ClearLog"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				]

				// save button
				+SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(4,2)
				[
					SAssignNew(SaveButton, SButton)
					.OnClicked(this, &SCustomLaunchOutputLog::OnSaveClicked)
					.Visibility( FDesktopPlatformModule::Get() != nullptr ? EVisibility::Visible : EVisibility::Collapsed )
					.IsEnabled_Lambda([InModel]() { return (InModel->GetNumLogMessages() > 0); } )
					.ToolTipText(LOCTEXT("SaveButtonTip", "Save log to file"))
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Save"))
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					]
				]
			]
		]
	];

	bIsUserScrolled = false;
	RequestForceScroll();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchOutputLog::CreateFilterWidget()
{
	return SNew(SHorizontalBox)
	.Visibility_Lambda( [this]() { return GetParentWidget()->GetVisibility(); } )

	// search/filter box
	+SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SSearchBox)
			.OnTextChanged(this, &SCustomLaunchOutputLog::OnFilterTextChanged)
			.OnTextCommitted(this, &SCustomLaunchOutputLog::OnFilterTextCommitted)
			.HintText(LOCTEXT("FilterTextHint", "Text Filter"))
			.InitialText(CurrentFilterText)
			.MinDesiredWidth(128)
			.ToolTipText(LOCTEXT("FilterTextTip", "Only show lines that contain the specified search text") )
			.DelayChangeNotificationsWhileTyping(true)
	]

	// filter button
	+SHorizontalBox::Slot()
	.AutoWidth()
	[
		SNew(SComboButton)
		.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButton")
		.ToolTipText(LOCTEXT("AddFilterToolTip", "Add an output log filter."))
		.OnGetMenuContent(this, &SCustomLaunchOutputLog::MakeFilterMenu)
		.MenuPlacement(MenuPlacement_BelowRightAnchor)
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Filter"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0, 0, 0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Filters", "Filters"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		]
	]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



TSharedRef<SWidget> SCustomLaunchOutputLog::MakeFilterMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	const bool bCloseSelfOnly = false;
	const bool bSearchable = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr, nullptr, bCloseSelfOnly, &FCoreStyle::Get(), bSearchable );

	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterAllLabel", "All Messages"),
		LOCTEXT("FilterAllTip", "Show all messages"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SCustomLaunchOutputLog::OnFilterChanged, ProjectLauncher::ELogFilter::All),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda( [this]() { return (LaunchLogTextMarshaller->GetFilter() == ProjectLauncher::ELogFilter::All) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterWarningsAndErrorsLabel", "Warnings & Errors"),
		LOCTEXT("FilterWarningsAndErrorsTip", "Show only warnings and errors"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SCustomLaunchOutputLog::OnFilterChanged, ProjectLauncher::ELogFilter::WarningsAndErrors),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda( [this]() { return (LaunchLogTextMarshaller->GetFilter() == ProjectLauncher::ELogFilter::WarningsAndErrors) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("FilterErrorsLabel", "Errors"),
		LOCTEXT("FilterErrorsTip", "Show only errors"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SCustomLaunchOutputLog::OnFilterChanged, ProjectLauncher::ELogFilter::Errors),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda( [this]() { return (LaunchLogTextMarshaller->GetFilter() == ProjectLauncher::ELogFilter::Errors) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SearchFiltersLogLabel", "Only Matching Lines"),
		LOCTEXT("SearchFiltersLogTip", "Only show log lines that match the text search filter"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SCustomLaunchOutputLog::OnSearchBoxFiltersLogToggle ),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda( [this]() { return bSearchBoxFiltersLog ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
		),
		NAME_None,
		EUserInterfaceActionType::Check);

	MenuBuilder.AddMenuSeparator();

	MenuBuilder.AddMenuEntry(
		LOCTEXT("WordWrapLabel", "Enable Word Wrapping"),
		LOCTEXT("WordWrapTip", "Split long log entries across multiple lines"),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateSP(this, &SCustomLaunchOutputLog::OnWordWrapToggle ),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda( [this]() { return bWordWrap ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
		),
		NAME_None,
		EUserInterfaceActionType::Check);

	return MenuBuilder.MakeWidget();
}


void SCustomLaunchOutputLog::ExtendTextBoxMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("LogMenuSectionLabel", "Output Log"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ClearLogLabel", "Clear"),
		LOCTEXT("ClearLogTip", "Clears all log messages"),
		FSlateIcon(FProjectLauncherStyle::GetStyleSetName(), "Icons.ClearLog"),
		FUIAction(
			FExecuteAction::CreateSP(this, &SCustomLaunchOutputLog::ClearLog ),
			FCanExecuteAction::CreateLambda([this]() { return Model->GetNumLogMessages() > 0; } ),
			FGetActionCheckState()
		),
		NAME_None,
		EUserInterfaceActionType::Check);


	MenuBuilder.AddMenuEntry(
		LOCTEXT("SageLogLabel", "Save As..."),
		LOCTEXT("SageLogTip", "Save log to file"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Save"),
		FUIAction(
			FExecuteAction::CreateSP(this, &SCustomLaunchOutputLog::SaveLog ),
			FCanExecuteAction::CreateLambda([this]() { return Model->GetNumLogMessages() > 0; } ),
			FGetActionCheckState()
		));

	MenuBuilder.EndSection();
}



void SCustomLaunchOutputLog::OnFilterTextChanged(const FText& FilterText)
{
	CurrentFilterText = FilterText;

	// only show matching lines in the log
	if (bSearchBoxFiltersLog)
	{
		LogMessageTextBox->GoTo(FTextLocation(0));
		LaunchLogTextMarshaller->SetFilterString(FilterText.ToString());
		LogMessageTextBox->Refresh();
		RequestForceScroll();
	}

	// highlight the first item
	LogMessageTextBox->BeginSearch(FilterText);
}

void SCustomLaunchOutputLog::OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	bool bFilterChanged = !InFilterText.EqualTo(CurrentFilterText);
	if (bFilterChanged)
	{
		OnFilterTextChanged(InFilterText);
	}
	else if (InCommitType == ETextCommit::OnEnter)
	{
		bool bReverse = FSlateApplication::Get().GetModifierKeys().IsShiftDown();
		bIsUserScrolled = true;
		LogMessageTextBox->AdvanceSearch(bReverse);
	}
}

void SCustomLaunchOutputLog::OnFilterChanged(ProjectLauncher::ELogFilter Filter)
{
	LogMessageTextBox->GoTo(FTextLocation(0));
	LaunchLogTextMarshaller->SetFilter(Filter);
	LogMessageTextBox->Refresh();
	RequestForceScroll();
}

void SCustomLaunchOutputLog::OnWordWrapToggle()
{
	bWordWrap = !bWordWrap;
	RequestForceScroll(true);
}

void SCustomLaunchOutputLog::OnSearchBoxFiltersLogToggle()
{
	bSearchBoxFiltersLog = !bSearchBoxFiltersLog;
	if (!bSearchBoxFiltersLog)
	{
		LaunchLogTextMarshaller->SetFilterString(FString());
	}
	else
	{
		LaunchLogTextMarshaller->SetFilterString(CurrentFilterText.ToString());
	}
	RefreshLog();
}

void SCustomLaunchOutputLog::OnUserScrolled(float ScrollOffset)
{
	bIsUserScrolled = ScrollOffset < 1.0 && !FMath::IsNearlyEqual(ScrollOffset, 1.0f);
}



void SCustomLaunchOutputLog::RefreshLog()
{
	LogMessageTextBox->GoTo(FTextLocation(0));
	LaunchLogTextMarshaller->RefreshAllLogMessages();
	LogMessageTextBox->Refresh();
	RequestForceScroll();
}

void SCustomLaunchOutputLog::RequestForceScroll(bool bIfUserHasNotScrolledUp)
{
	if (LaunchLogTextMarshaller->GetNumFilteredMessages() > 0
		&& !LogMessageTextBox->AnyTextSelected()
		&& (!bIfUserHasNotScrolledUp || !bIsUserScrolled))
	{
		LogMessageTextBox->ScrollTo(ETextLocation::EndOfDocument);
		bIsUserScrolled = false;
	}
}




FReply SCustomLaunchOutputLog::OnSaveClicked()
{
	SaveLog();
	return FReply::Handled();
}



FReply SCustomLaunchOutputLog::OnClearClicked()
{
	ClearLog();
	return FReply::Handled();
}



FReply SCustomLaunchOutputLog::OnCopyClicked()
{
	CopyLog();
	return FReply::Handled();
}


FString SCustomLaunchOutputLog::GetLogAsString( bool bSelectedLinesOnly ) const
{
	if (bSelectedLinesOnly)
	{
		return LogMessageTextBox->GetSelectedText().ToString();
	}
	else
	{
		return LogMessageTextBox->GetPlainText().ToString();
	}
}




void SCustomLaunchOutputLog::ClearLog()
{
	Model->ClearLogMessages();
	RefreshLog();
	bIsUserScrolled = false;
}

void SCustomLaunchOutputLog::CopyLog()
{
	FString LogText = GetLogAsString(false);
	if (!LogText.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy( *LogText );
	}
}



void SCustomLaunchOutputLog::SaveLog()
{
	static FString LastSavePath;

	// @todo: try using ISlateFileDialogsModule
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform != nullptr)
	{
		TArray<FString> FileNames;

		if (DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("SaveLogDialogTitle", "Save Log As...").ToString(),
			*LastSavePath,
			TEXT("BuildCookRun.log"),
			TEXT("Log Files (*.log)|*.log"),
			EFileDialogFlags::None,
			FileNames))
		{
			if (FileNames.Num() > 0)
			{
				FString FileName = FileNames[0];
				LastSavePath = FPaths::GetPath(FileName); // record last used directory
				if (FPaths::GetExtension(FileName).IsEmpty()) // make sure there's an extension
				{
					FileName += TEXT(".log");
				}

				if (!FFileHelper::SaveStringToFile( GetLogAsString(false), *FileName))
				{
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SaveLogDialogFail", "Failed to save the log"));
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE
