// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "Model/ProjectLauncherModel.h"
#include "Widgets/Output/CustomLaunchOutputLogMarshaller.h"

class SWidget;
class SButton;
class SMultiLineEditableTextBox;

class SCustomLaunchOutputLog 
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomLaunchOutputLog) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel, const TSharedRef<ProjectLauncher::FLaunchLogTextLayoutMarshaller>& InLaunchLogTextMarshaller);

	void CopyLog();
	void SaveLog();
	void ClearLog();
	FString GetLogAsString( bool bSelectedLinesOnly = false ) const;

	void RefreshLog();

	void RequestForceScroll( bool bIfUserHasNotScrolledUp = false );

	TSharedRef<SWidget> CreateFilterWidget();

private:
	TSharedRef<SWidget> MakeFilterMenu();
	FReply OnSaveClicked();
	FReply OnClearClicked();
	FReply OnCopyClicked();

	void OnFilterTextChanged(const FText& FilterText);
	void OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType);
	void OnFilterChanged(ProjectLauncher::ELogFilter Filter);
	void OnWordWrapToggle();
	void OnSearchBoxFiltersLogToggle();
	void OnUserScrolled(float ScrollOffset);
	void ExtendTextBoxMenu(class FMenuBuilder& Builder);

	TSharedPtr<SMultiLineEditableTextBox> LogMessageTextBox;
	TSharedPtr<SButton> SaveButton;
	TSharedPtr<SButton> ClearButton;
	TSharedPtr<SButton> CopyButton;

private:
	bool bIsUserScrolled = false;
	bool bWordWrap = false;
	bool bSearchBoxFiltersLog = true;
	FText CurrentFilterText;
	TSharedPtr<ProjectLauncher::FLaunchLogTextLayoutMarshaller> LaunchLogTextMarshaller;
	TSharedPtr<ProjectLauncher::FModel> Model;
};
