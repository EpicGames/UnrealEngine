// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Profiles/SCustomLaunchCustomProfileEditor.h"

#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/ProjectLauncherStyle.h"
#include "Extension/LaunchExtension.h"
#include "Extension/BuildCookRunCommandExtension.h"
#include "Extension/UATCommandLaunchExtension.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Shared/SCustomLaunchDeviceCombo.h"
#include "Widgets/Shared/SCustomLaunchDeviceListView.h"
#include "SPositiveActionButton.h"
#include "DesktopPlatformModule.h"
#include "PlatformInfo.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "InstalledPlatformInfo.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/ConfigCacheIni.h"
#include "GameProjectHelper.h"


#define LOCTEXT_NAMESPACE "SCustomLaunchCustomProfileEditor"

namespace ProjectLauncher
{
	extern TSharedRef<ILaunchProfileTreeBuilder> CreateTreeBuilder( const ILauncherProfilePtr& InProfile, TSharedRef<ProjectLauncher::FModel> InModel );
};

class FProfileSectionDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FProfileSectionDragDropOp, FDragDropOperation)

	static TSharedRef<FProfileSectionDragDropOp> New(const ProjectLauncher::FLaunchProfileTreeNodePtr& InDraggedNode)
	{
		TSharedRef<FProfileSectionDragDropOp> Operation = MakeShared<FProfileSectionDragDropOp>();
		Operation->DraggedNode = InDraggedNode;
		Operation->Construct();
		return Operation;
	}

	ProjectLauncher::FLaunchProfileTreeNodePtr GetDraggeedNode() const { return DraggedNode; }

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
			.Padding(8, 8, 200, 8)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(FText::FromString(DraggedNode->UATCommand->GetDescription()))
				.Font(FCoreStyle::Get().GetFontStyle("SmallFontBold"))
			];
	}

private:
	ProjectLauncher::FLaunchProfileTreeNodePtr DraggedNode;
};




class SLaunchProfileCategoryTreeRow : public SLaunchProfileTreeRow
{
public:
	SLATE_BEGIN_ARGS(SLaunchProfileCategoryTreeRow){}
	SLATE_END_ARGS()

	ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode;
	bool bIsPrimaryHeading;
	bool bIsRootHeading;
	SCustomLaunchCustomProfileEditor* ProfileEditor;

	void Construct(const FArguments& InArgs, ProjectLauncher::FLaunchProfileTreeNodePtr InTreeNode, SCustomLaunchCustomProfileEditor* InProfileEditor, bool bInIsPrimaryHeading, bool bInIsRootHeading, bool bPinned )
	{
		TreeNode = InTreeNode;
		ProfileEditor = InProfileEditor;
		bIsPrimaryHeading = bInIsPrimaryHeading;
		bIsRootHeading = bInIsRootHeading;

		auto GetErrorVisibility = [this]()
		{
			if (!IsItemExpanded())
			{
				for (ProjectLauncher::FLaunchProfileTreeNodePtr ChildNode : TreeNode->Children)
				{
					if (ChildNode->Callbacks.Validation.HasError( TreeNode->GetTreeData()->Profile.ToSharedRef(), ChildNode->UATCommand ))
					{
						return EVisibility::Visible;
					}
				}
			}
			
			return EVisibility::Collapsed;
		};

		auto GetErrorToolTipText = [this]()
		{
			TArray<FString> ErrorLines;
			for (ProjectLauncher::FLaunchProfileTreeNodePtr ChildNode : TreeNode->Children)
			{
				ChildNode->Callbacks.Validation.GetErrorText( TreeNode->GetTreeData()->Profile.ToSharedRef(), ChildNode->UATCommand, ErrorLines );
			}

			FTextBuilder TextBuilder;
			for (const FString& ErrorLine : ErrorLines)
			{
				TextBuilder.AppendLine(FText::FromString(ErrorLine));
			}

			return TextBuilder.ToText();
		};

		auto GetUATCommandEnabledCheckState = [this]()
		{
			if (TreeNode->UATCommand->IsEnabled())
			{
				return ECheckBoxState::Checked;
			}
			
			return ECheckBoxState::Unchecked;
		};

		auto SetUATCommandEnabledCheckState = [this](ECheckBoxState State)
		{
			TreeNode->UATCommand->SetEnabled( State == ECheckBoxState::Checked);
			ProfileEditor->TreeBuilder->OnPropertyChanged();
		};

		auto GetUATCommandDescription = [this]()
		{
			return FText::FromString(TreeNode->UATCommand->GetDescription());
		};

		auto SetUATCommandDescription = [this]( const FText& NewValue, ETextCommit::Type )
		{ 
			TreeNode->UATCommand->SetDescription(NewValue.ToString()); 
			ProfileEditor->TreeBuilder->OnPropertyChanged();
		};

		auto OnRenameUATCommandClicked = [this]()
		{
			ProfileEditor->BeginRenameUATCommand(TreeNode->UATCommand.ToSharedRef());
			return FReply::Handled();
		};

		auto GetUATCommandBrush = [this]()
		{
			return ProfileEditor->GetIconForUATCommand(TreeNode->UATCommand.ToSharedRef()).GetIcon();
		};


		TSharedPtr<SHorizontalBox> HeaderBar;
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
			.Padding(FMargin(0, bIsPrimaryHeading ? 8 : 0, 0, 1))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(bPinned ? "Brushes.Panel" : "Brushes.Header"))
				.Padding(8*TreeNode->Depth, 0, bPinned ? 8 : 0, 0)
				[
					SNew(SBox)
					.MinDesiredHeight(26.0f)
					[
						SAssignNew(HeaderBar, SHorizontalBox)

						// tree expander arrow
						+SHorizontalBox::Slot()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(2, 0, 0, 0)
						.AutoWidth()
						[
							SNew(SExpanderArrow, SharedThis(this))
							.StyleSet( &FCoreStyle::Get() )
							.Visibility( bPinned ? EVisibility::Hidden : EVisibility::Visible )
						]
					]
				]
			]
		];

		if (bIsPrimaryHeading && !bPinned)
		{
			// checkbox to enable the UAT command
			HeaderBar->AddSlot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(4,0,0,0)
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsEnabled_Lambda([this]() { return TreeNode->GetTreeData()->Profile->GetUATCommands().Num() > 1; })
				.IsChecked_Lambda(GetUATCommandEnabledCheckState)
				.OnCheckStateChanged_Lambda(SetUATCommandEnabledCheckState)
			];
		}

		// error message indicator - only shown when the heading is collapsed to indicate a child node has an error
		HeaderBar->AddSlot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(2, 0)
		.AutoWidth()
		[
			SNew(SImage)
			.Image(FProjectLauncherStyle::Get().GetBrush("Icons.WarningWithColor.Small"))
			.Visibility_Lambda(GetErrorVisibility)
			.ToolTipText_Lambda(GetErrorToolTipText)
		];


		if (bIsPrimaryHeading)
		{
			// icon
			HeaderBar->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(4, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SImage)
				.Image(GetUATCommandBrush())
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];

			// inline editable name field for UAT commands etc
			TSharedPtr<SInlineEditableTextBlock> RenameBox;
			HeaderBar->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(4, 0, 0, 0)
			.AutoWidth()
			[
				SAssignNew(RenameBox,SInlineEditableTextBlock)
				.Text_Lambda(GetUATCommandDescription)
				.OnTextCommitted_Lambda(SetUATCommandDescription)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFontBold"))
			];
			ProfileEditor->UATCommandNameFields.Add(TreeNode->UATCommand, RenameBox);
		}
		else
		{
			// basic display name
			HeaderBar->AddSlot()
			.VAlign(VAlign_Center)
			.Padding(4, 0, 0, 0)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(TreeNode->Name)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFontBold"))
			];
		}

		if (bIsRootHeading && !bPinned)
		{
			// spacer to push everthing else to the right
			HeaderBar->AddSlot()
			.FillWidth(1)
			[
				SNew(SSpacer)
			];

			// drop-down menu
			HeaderBar->AddSlot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2, 0, 6, 0)
			.AutoWidth()
			[
				SNew(SComboButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.HasDownArrow(false)
				.OnGetMenuContent( ProfileEditor, &SCustomLaunchCustomProfileEditor::MakeTreeNodeSubMenuWidget, TreeNode)
				.ButtonContent()
				[
					SNew(SImage)
					.Image(FProjectLauncherStyle::Get().GetBrush("OverflowButton"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			];
		}

		SLaunchProfileTreeRow::ConstructInternal(
			STableRow::FArguments()
			.OnDragDetected(this, &SLaunchProfileCategoryTreeRow::HandleDragDetected)		
			.OnCanAcceptDrop(this, &SLaunchProfileCategoryTreeRow::HandleCanAcceptDrop)
			.OnAcceptDrop(this, &SLaunchProfileCategoryTreeRow::HandleAcceptDrop)
			.ShowSelection(false),
			ProfileEditor->GetTreeView()
		);
	}

	virtual FReply OnMouseButtonUp(const FGeometry& Geometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton && bIsRootHeading)
		{
			FMenuBuilder MenuBuilder(true, nullptr);
			ProfileEditor->MakeTreeNodeSubMenu(MenuBuilder, TreeNode);

			FSlateApplication::Get().PushMenu( SharedThis(this),
				FWidgetPath(),
				MenuBuilder.MakeWidget(),
				MouseEvent.GetScreenSpacePosition(),
				FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

			return FReply::Handled();
		}

		return SLaunchProfileTreeRow::OnMouseButtonUp(Geometry, MouseEvent);
	}

	FReply HandleDragDetected(const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		if (bIsPrimaryHeading && MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			TSharedPtr<FDragDropOperation> DragDropOp = FProfileSectionDragDropOp::New(TreeNode);
			return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
		}

		return FReply::Unhandled();
	}

	TOptional<EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, ProjectLauncher::FLaunchProfileTreeNodePtr InTreeItem)
	{
		TSharedPtr<FProfileSectionDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FProfileSectionDragDropOp>();
		if (DragDropOp.IsValid() && bIsPrimaryHeading && InItemDropZone != EItemDropZone::OntoItem)
		{
			return TOptional<EItemDropZone>(InItemDropZone);
		}

		return TOptional<EItemDropZone>();
	}


	FReply HandleAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InItemDropZone, ProjectLauncher::FLaunchProfileTreeNodePtr InTreeItem)
	{
		TSharedPtr<FProfileSectionDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FProfileSectionDragDropOp>();
		if (DragDropOp.IsValid())
		{
			ProfileEditor->ReorderNodes(DragDropOp->GetDraggeedNode(), TreeNode, InItemDropZone);
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

};

class SLaunchProfileFullWidthTreeRow : public SLaunchProfileTreeRow
{
public:
	SLATE_BEGIN_ARGS(SLaunchProfileFullWidthTreeRow){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode, SCustomLaunchCustomProfileEditor* ProfileEditor )
	{
		this->ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.Padding(FMargin(0, 0, 1, 0))
			[
				SNew(SBorder)
				.BorderImage_Lambda( [this]() { return IsHovered() ? FAppStyle::Get().GetBrush("Brushes.Header") : FAppStyle::Get().GetBrush("Brushes.Panel"); } )
				.Padding(1)
				[
					SNew(SBox)
					.Padding( FMargin(24*TreeNode->Depth, 1, 24, 1))
					.VAlign(VAlign_Center)
					[
						TreeNode->Widget.ToSharedRef()
					]
				]
			]
		];

		SLaunchProfileTreeRow::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(false),
			ProfileEditor->GetTreeView()
		);
	}
};


class SLaunchProfilePropertyTreeRow : public SLaunchProfileTreeRow
{
public:
	DECLARE_DELEGATE_OneParam( FOnSplitterResized, float );


	SLATE_BEGIN_ARGS(SLaunchProfilePropertyTreeRow){}
		SLATE_ATTRIBUTE(float, SplitterValue)
		SLATE_EVENT(FOnSplitterResized, OnSplitterResized)
	SLATE_END_ARGS()

	BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	void Construct(const FArguments& InArgs, ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode, SCustomLaunchCustomProfileEditor* ProfileEditor )
	{
		SplitterValue = InArgs._SplitterValue;
		OnSplitterResized = InArgs._OnSplitterResized;


		TSharedRef<SWidget> OptionalResetToDefaultWidget = SNullWidget::NullWidget;
		if (TreeNode->Callbacks.IsDefault != nullptr && TreeNode->Callbacks.SetToDefault != nullptr)
		{
			auto IsVisible = [TreeNode]()
			{
				if (TreeNode->Callbacks.IsDefault())
				{
					return false;
				}

				if (TreeNode->Callbacks.IsEnabled && !TreeNode->Callbacks.IsEnabled())
				{
					return false;
				}

				return true;
			};

			auto SetToDefault = [TreeNode]()
			{
				TreeNode->Callbacks.SetToDefault();
				TreeNode->GetTreeData()->TreeBuilder->OnPropertyChanged();
				return FReply::Handled();
			};

			OptionalResetToDefaultWidget = SNew(SButton)
				.ButtonStyle(FProjectLauncherStyle::Get(), "HoverHintOnly")
				.ToolTipText( LOCTEXT("ResetToDefaultToolTip", "Reset this property to its default value.") )
				.Visibility_Lambda( [IsVisible]() { return IsVisible() ? EVisibility::Visible : EVisibility::Collapsed; } )
				.OnClicked_Lambda(SetToDefault)
				.ContentPadding(0)
				[
					SNew(SImage)
					.Image( FProjectLauncherStyle::Get().GetBrush("Icons.DiffersFromDefault") )
					.DesiredSizeOverride(FVector2D(16,16))
					.ColorAndOpacity( FSlateColor::UseForeground() )
				]
			;
		}

		auto GetErrorVisibility = [TreeNode]()
		{
			if (TreeNode->Callbacks.Validation.HasError( TreeNode->GetTreeData()->Profile.ToSharedRef(), TreeNode->UATCommand ))
			{
				return EVisibility::Visible;
			}
			
			return EVisibility::Collapsed;
		};

		auto GetErrorToolTipText = [TreeNode]()
		{
			TArray<FString> ErrorLines;
			TreeNode->Callbacks.Validation.GetErrorText( TreeNode->GetTreeData()->Profile.ToSharedRef(), TreeNode->UATCommand, ErrorLines );

			FTextBuilder TextBuilder;
			for (const FString& ErrorLine : ErrorLines)
			{
				TextBuilder.AppendLine(FText::FromString(ErrorLine));
			}

			return TextBuilder.ToText();
		};

		auto MakeSplitterSlot = [this]( TSharedRef<SWidget> SlotContent )
		{
			return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.Padding(FMargin(0, 0, 0, 1))
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				[
					SNew( SBorder )
					.BorderImage_Lambda( [this]() { return IsHovered() ? FAppStyle::Get().GetBrush("Brushes.Header") : FAppStyle::Get().GetBrush("Brushes.Panel"); } )
					.Padding(0)
					[
						SlotContent
					]
				]
			];
		};

		TSharedPtr<SHorizontalBox> PropertyNameBox;


		this->ChildSlot
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(1.0)
			.HitDetectionSplitterHandleSize(5.0)
			.MinimumSlotHeight(26.0f)

			// property name
			+SSplitter::Slot()
			.Value_Lambda( [this]() { return 1.0f - SplitterValue.Get(); } )
			.OnSlotResized_Lambda( [this](float NewPos) { OnSplitterResized.ExecuteIfBound(1.0f - FMath::Clamp(NewPos,0.0f, 1.0f)); } )
			.MinSize(8)
			[
				MakeSplitterSlot(
					SNew(SBox)
					.Padding( FMargin(8 + (16*TreeNode->Depth), 1, 4, 0))
					.VAlign(VAlign_Top)
					[
						SAssignNew(PropertyNameBox, SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Left)
						.VAlign(VAlign_Center)
						.Padding(0,4)
						[
							SNew(STextBlock)
							.Text(TreeNode->Name)
							.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
						]
					]
				)
			]

			// property value
			+SSplitter::Slot()
			.Value_Lambda( [this](){ return SplitterValue.Get(); } )
			.OnSlotResized_Lambda( [this](float NewPos) { OnSplitterResized.ExecuteIfBound(FMath::Clamp(NewPos,0.0f, 1.0f)); } )
			.MinSize(32)
			[
				MakeSplitterSlot(
					SNew(SBox)
					.Padding( FMargin(8, 1, 1, 1))
					.VAlign(VAlign_Center)
					.IsEnabled_Lambda( [TreeNode]() { return TreeNode->Callbacks.IsEnabled ? TreeNode->Callbacks.IsEnabled() : true; } )
					[
						TreeNode->Widget.ToSharedRef()
					]
				)
			]

			// reset to default button
			+SSplitter::Slot()
			.MinSize(24)
			.Resizable(false)
			.Value(0)
			[
				MakeSplitterSlot(
					SNew(SBox)
					.Padding(0)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						OptionalResetToDefaultWidget
					]
				)
			]
		];

		if (TreeNode->Callbacks.Validation.IsSet() )
		{
			PropertyNameBox->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(2, 1, 0, 0)
			[
				SNew(SImage)
				.Image(FProjectLauncherStyle::Get().GetBrush("Icons.WarningWithColor.Small"))
				.Visibility_Lambda(GetErrorVisibility)
				.ToolTipText_Lambda(GetErrorToolTipText)
			];
		}


		SLaunchProfileTreeRow::ConstructInternal(
			STableRow::FArguments()
			.ShowSelection(false),
			ProfileEditor->GetTreeView()
		);
	}
	END_SLATE_FUNCTION_BUILD_OPTIMIZATION


	FOnSplitterResized OnSplitterResized;
	TAttribute<float> SplitterValue;
};



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchCustomProfileEditor::Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel)
{
	Model = InModel;

	TreeBuilder = ProjectLauncher::CreateTreeBuilder( nullptr, InModel );

	auto GetErrorVisibility = [this]()
	{
		if (CurrentProfile.IsValid() && (!CurrentProfile->IsValidForLaunch() || CurrentProfile->GetAllCustomWarnings().Num() > 0))
		{
			return EVisibility::Visible;
		}
			
		return EVisibility::Collapsed;
	};

	auto GetErrorText = [this]()
	{
		return ProjectLauncher::GetProfileLaunchErrorMessage(CurrentProfile);
	};



	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ChildWindow.Background"))
		.Padding(2)
		[
			SNew(SVerticalBox)

			// main property tree
			+SVerticalBox::Slot()
			.FillHeight(1)
			.Padding(2)
			[
				SAssignNew(TreeView, SLaunchProfileTreeView)
				.TreeItemsSource(&TreeBuilder->GetProfileTree()->Nodes)
				.SelectionMode( ESelectionMode::Single )
				.OnGenerateRow( this, &SCustomLaunchCustomProfileEditor::OnGenerateWidgetForTreeNode, false )
				.OnGeneratePinnedRow( this, &SCustomLaunchCustomProfileEditor::OnGenerateWidgetForTreeNode, true )
				.OnGetChildren( this, &SCustomLaunchCustomProfileEditor::OnGetChildren)
				.HandleDirectionalNavigation(false)
				//.ShouldStackHierarchyHeaders(true) // keep headings at the top when scrolling (@todo: disabled as it causes scrolling judder - bug in STreeView)
			]

			// error banner
			+SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Fill)
			.Padding(2)
			[
				SNew(SBorder)
				.Visibility_Lambda(GetErrorVisibility)
				.Padding(8)

				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					.Padding(4)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.WarningWithColor.Large"))
						.DesiredSizeOverride(FVector2D(24,24))
					]

					+SHorizontalBox::Slot()
					.FillWidth(1)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.Padding(4)
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Text_Lambda(GetErrorText)
					]
				]
			]
		]
	];

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedRef<ITableRow> SCustomLaunchCustomProfileEditor::OnGenerateWidgetForTreeNode( ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode, const TSharedRef<STableViewBase>& OwnerTable, bool bPinned )
{
	if (TreeNode->Name.IsEmpty() && TreeNode->Widget.IsValid())
	{
		return SNew(SLaunchProfileFullWidthTreeRow, TreeNode, this)
		.IsEnabled_Lambda( [TreeNode]() { return TreeNode->Callbacks.IsEnabled ? TreeNode->Callbacks.IsEnabled() : true; } )
		;
	}
	else if (TreeNode->Widget.IsValid())
	{
		return SNew(SLaunchProfilePropertyTreeRow, TreeNode, this)
		.SplitterValue_Lambda( [this]() { return SplitterPos; } )
		.OnSplitterResized_Lambda( [this](float NewPos) { SplitterPos = FMath::Clamp(NewPos, 0.0f, 1.0f); } )
		;
	}
	else
	{
		bool bIsPrimaryHeading = TreeNode->UATCommand.IsValid() && TreeBuilder->GetProfileTree()->UATCommandHeadings.Contains(TreeNode);
		bool bIsRootHeading = TreeBuilder->GetProfileTree()->Nodes.Contains(TreeNode);
		return SNew(SLaunchProfileCategoryTreeRow, TreeNode, this, bIsPrimaryHeading, bIsRootHeading, bPinned);
	}
}

void SCustomLaunchCustomProfileEditor::ReorderNodes(ProjectLauncher::FLaunchProfileTreeNodePtr SourceNode, ProjectLauncher::FLaunchProfileTreeNodePtr TargetNode, EItemDropZone DropZone)
{
	// do nothing if we're dropped on ourselves
	if (SourceNode == TargetNode)
	{
		return;
	}

	TArray<ProjectLauncher::FLaunchProfileTreeNodePtr>& Nodes = TreeBuilder->GetProfileTree()->Nodes;
	if (!Nodes.Contains(SourceNode) || !Nodes.Contains(TargetNode))
	{
		return;
	}

	// move the item to where we want it
	Nodes.Remove(SourceNode);
	int32 TargetIndex = Nodes.IndexOfByKey(TargetNode);
	if (DropZone != EItemDropZone::AboveItem)
	{
		TargetIndex++;
	}
	Nodes.Insert(SourceNode, TargetIndex);

	// ensure the order of items is contiguous
	// @todo: maybe do away with the magic numbers for initial ordering?
	int32 PrevSortOrder = INT32_MIN;
	for ( const ProjectLauncher::FLaunchProfileTreeNodePtr& Node : Nodes)
	{
		if (Node->SortOrder <= PrevSortOrder && Node->SortOrder != INT32_MIN)
		{
			Node->SortOrder = PrevSortOrder+1;
			if (Node->UATCommand.IsValid())
			{
				Node->UATCommand->SetOrder(Node->SortOrder);
			}
		}

		PrevSortOrder = Node->SortOrder;
	}

	TreeBuilder->OnPropertyChanged();
	TreeView->RequestTreeRefresh();
}

void SCustomLaunchCustomProfileEditor::BeginRenameUATCommand( const ILauncherProfileUATCommandRef& UATCommand ) const
{
	const TWeakPtr<SInlineEditableTextBlock>* RenameBoxPtr = UATCommandNameFields.Find(UATCommand.ToSharedPtr());
	if (RenameBoxPtr != nullptr)
	{
		ExecuteOnGameThread( UE_SOURCE_LOCATION, [RenameBox = (*RenameBoxPtr)]
		{
			if (RenameBox.IsValid())
			{
				RenameBox.Pin()->EnterEditingMode(); 
			}
		});
	}
}

bool SCustomLaunchCustomProfileEditor::IsHierarchyVisible(ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode) const
{
	// don't show empty headings
	if (!TreeNode->Name.IsEmpty() && !TreeNode->Widget.IsValid() && TreeNode->Children.Num() == 0)
	{
		return false;
	}

	// don't show if we are invisible
	if (TreeNode->Callbacks.IsVisible.IsSet() && !TreeNode->Callbacks.IsVisible() )
	{
		return false;
	}

	// no children - show by default
	if (TreeNode->Children.Num() == 0)
	{
		return true;
	}

	// only show if we have a visible child
	for ( const ProjectLauncher::FLaunchProfileTreeNodePtr& Child : TreeNode->Children)
	{
		if (IsHierarchyVisible(Child))
		{
			return true;
		}
	}

	return false;
}


void SCustomLaunchCustomProfileEditor::OnGetChildren(ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode, TArray<ProjectLauncher::FLaunchProfileTreeNodePtr>& OutChildren)
{
	for ( const ProjectLauncher::FLaunchProfileTreeNodePtr& Child : TreeNode->Children)
	{
		if (IsHierarchyVisible(Child))
		{
			OutChildren.Add(Child);
		}
	}
}

FVector2D SCustomLaunchCustomProfileEditor::GetScrollDistance()
{
	if (!TreeView.IsValid() || !TreeBuilder.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return TreeView->GetScrollDistance();
}

FVector2D SCustomLaunchCustomProfileEditor::GetScrollDistanceRemaining()
{
	if (!TreeView.IsValid() || !TreeBuilder.IsValid())
	{
		return FVector2D::ZeroVector;
	}

	return TreeView->GetScrollDistanceRemaining();
}

TSharedRef<SWidget> SCustomLaunchCustomProfileEditor::GetScrollWidget()
{
	return SharedThis(this);
}


void SCustomLaunchCustomProfileEditor::SetProfile( const ILauncherProfilePtr& Profile )
{
	// save the current node collapsed state (nb. not saved to the profile)
	if (CurrentProfile.IsValid() && TreeBuilder.IsValid())
	{
		TSet<FString>& CollapsedState = CachedCollapsedNodes.Add(CurrentProfile);
		for ( TTuple<FString,ProjectLauncher::FLaunchProfileTreeNodePtr> Pair : TreeBuilder->GetProfileTree()->HeadingNodes)
		{
			CacheCollapsedRecursive(CollapsedState, Pair.Key, Pair.Value);
		}
	}

	// change the profile and rebuild
	CurrentProfile = Profile;
	TreeBuilder = ProjectLauncher::CreateTreeBuilder( Profile, Model.ToSharedRef() );
	TreeView->SetTreeItemsSource( &TreeBuilder->GetProfileTree()->Nodes );
	TreeView->RequestTreeRefresh();

	// restore the collapsed state if we have one saved, otherwise expand everything (need to defer until after the tree is rebuilt)
	ExecuteOnGameThread( UE_SOURCE_LOCATION, [this]
	{
		TSet<FString> CollapsedState;
		if (CachedCollapsedNodes.Contains(CurrentProfile))
		{
			CollapsedState = CachedCollapsedNodes.FindAndRemoveChecked(CurrentProfile);
		}
		for ( TTuple<FString,ProjectLauncher::FLaunchProfileTreeNodePtr> Pair : TreeBuilder->GetProfileTree()->HeadingNodes)
		{
			ApplyCollapsedRecusive(CollapsedState, Pair.Key, Pair.Value);
		}
	});

}

void SCustomLaunchCustomProfileEditor::RebuildTree()
{
	if (!CurrentProfile.IsValid() || !TreeView.IsValid())
	{
		return;
	}

	// cache which headings were collapsed before we rebuild the tree so we can restore the state in the new tree
	TSet<FString> CurrentlyCollapsed;
	for ( TTuple<FString,ProjectLauncher::FLaunchProfileTreeNodePtr> Pair : TreeBuilder->GetProfileTree()->HeadingNodes)
	{
		CacheCollapsedRecursive(CurrentlyCollapsed, Pair.Key, Pair.Value);
	}

	TreeView->ClearRootItemsSource();
	TreeBuilder.Reset();

	TreeBuilder = ProjectLauncher::CreateTreeBuilder( CurrentProfile, Model.ToSharedRef() );
	TreeView->SetTreeItemsSource( &TreeBuilder->GetProfileTree()->Nodes );
	TreeView->RequestTreeRefresh();

	// restore the collapsed state next tick, once the tree has refreshed
	PendingCollapseHeadings = MoveTemp(CurrentlyCollapsed);
}

void SCustomLaunchCustomProfileEditor::CacheCollapsedRecursive( TSet<FString>& Collapsed, const FString& NodeName, ProjectLauncher::FLaunchProfileTreeNodePtr Node) const
{
	if (TreeView->IsItemExpanded(Node))
	{
		for ( TTuple<FString, ProjectLauncher::FLaunchProfileTreeNodePtr> Pair : Node->SubHeadingNodes)
		{
			CacheCollapsedRecursive(Collapsed, NodeName + TEXT(":") + Pair.Key, Pair.Value);
		}
	}
	else
	{
		Collapsed.Add(NodeName);
	}
}

void SCustomLaunchCustomProfileEditor::ApplyCollapsedRecusive( const TSet<FString>& Collapsed, const FString& NodeName, ProjectLauncher::FLaunchProfileTreeNodePtr Node) const
{
	if (!Collapsed.Contains(NodeName))
	{
		TreeView->SetItemExpansion(Node, true);

		for ( TTuple<FString, ProjectLauncher::FLaunchProfileTreeNodePtr> Pair : Node->SubHeadingNodes)
		{
			ApplyCollapsedRecusive(Collapsed,  NodeName + TEXT(":") + Pair.Key, Pair.Value);
		}
	}
	else
	{
		TreeView->SetItemExpansion(Node, false);
	}
}


void SCustomLaunchCustomProfileEditor::MakeMenu( FMenuBuilder& MenuBuilder, EMenuType MenuType, ILauncherProfileBuildCookRunPtr BuildCookRun )
{
	ProjectLauncher::FLaunchProfileTreeDataRef TreeData = TreeBuilder->GetProfileTree();

	if (Model->CanUseSimplifiedLayout(CurrentProfile.ToSharedRef()) && !BuildCookRun.IsValid())
	{
		BuildCookRun = CurrentProfile->GetFirstBuildCookRun();
	}

	// ...build extension submenu structure...

	struct FExtensionMenuSubMenu
	{
		FText MenuEntryName;
		FText DisplayName;
		TArray<TSharedRef<ProjectLauncher::FLaunchExtension>> Extensions;
	};

	struct FExtensionMenuSection
	{
		FText DisplayName;
		TMap<FString,TSharedPtr<FExtensionMenuSubMenu>> SubMenus;
		TArray<TSharedRef<ProjectLauncher::FLaunchExtension>> Extensions;
		bool bIsDefault = false;
	};

	TMap<FString,FExtensionMenuSection> Sections;

	TArray<TSharedRef<ProjectLauncher::FLaunchExtension>> Extensions = ProjectLauncher::FLaunchExtension::GetExtensions();
	for (const TSharedRef<ProjectLauncher::FLaunchExtension>& Extension : Extensions)
	{
		ProjectLauncher::FLaunchExtension::FExtensionsMenuEntry ExtensionMenuEntry;
		Extension->GetExtensionsMenuEntry(ExtensionMenuEntry);
		if (ExtensionMenuEntry.Type == ProjectLauncher::FLaunchExtension::FExtensionsMenuEntry::Type_None)
		{
			continue;
		}

		// filter based on menu type (this prevents empty submenus)
		bool bIsUATCommand = Extension->IsUATCommandManager();
		if (bIsUATCommand != (MenuType == EMenuType::UATCommands))
		{
			continue;
		}

		// get section
		FString SectionName = ExtensionMenuEntry.SectionName;
		FText SectionDisplayName = ExtensionMenuEntry.SectionDisplayName;
		if (SectionName.IsEmpty())
		{
			SectionName = Extension->GetInternalName();
			SectionDisplayName = Extension->GetDisplayName();
		}
		FExtensionMenuSection* Section = Sections.Find(SectionName);
		if (Section == nullptr)
		{
			Section = &Sections.Add(SectionName);
			Section->DisplayName = SectionDisplayName;
		}

		if (ExtensionMenuEntry.Type == ProjectLauncher::FLaunchExtension::FExtensionsMenuEntry::Type_SubMenu)
		{
			// get submenu
			TSharedPtr<FExtensionMenuSubMenu>* SubMenu = Section->SubMenus.Find(ExtensionMenuEntry.SubmenuName);
			if (SubMenu == nullptr)
			{
				SubMenu = &Section->SubMenus.Add(ExtensionMenuEntry.SubmenuName, MakeShared<FExtensionMenuSubMenu>());
				(*SubMenu)->DisplayName = ExtensionMenuEntry.SubmenuDisplayName;
			}

			(*SubMenu)->Extensions.Add(Extension);
		}
		else
		{
			Section->bIsDefault |= ExtensionMenuEntry.bIsDefault;
			Section->Extensions.Add(Extension);
		}
	}


	// sort sections alphabetically, keeping the default section at th top
	Sections.ValueSort([](const FExtensionMenuSection& A, const FExtensionMenuSection& B)
	{
		if (A.bIsDefault != B.bIsDefault)
		{
			return A.bIsDefault;
		}

		return A.DisplayName.CompareToCaseIgnored(B.DisplayName) < 0; 
	});



	// ...construct the extensions menu...

	for (const auto& SectionItr : Sections)
	{
		// start the section
		MenuBuilder.BeginSection(NAME_None, SectionItr.Value.DisplayName);

		// add all extensions
		for (TSharedRef<ProjectLauncher::FLaunchExtension> Extension : SectionItr.Value.Extensions)
		{
			MakeExtensionMenu(MenuBuilder, MenuType, Extension, BuildCookRun);
		}

		// add all submenus
		for (const auto& SubMenuItr : SectionItr.Value.SubMenus)
		{
			auto MakeExtensionSubMenu = [this, MenuType, BuildCookRun](FMenuBuilder& MenuBuilder, TSharedPtr<FExtensionMenuSubMenu> SubMenu)
			{
				MenuBuilder.SetSearchable(false);
				for (TSharedRef<ProjectLauncher::FLaunchExtension> Extension : SubMenu->Extensions)
				{
					MakeExtensionMenu(MenuBuilder, MenuType, Extension, BuildCookRun);
				}
			};

			const bool bInOpenSubMenuOnClick = false;
			const bool bShouldCloseWindowAfterMenuSelection = true;

			MenuBuilder.AddSubMenu(
				SubMenuItr.Value->DisplayName,
				FText::GetEmpty(),
				FNewMenuDelegate::CreateLambda( MakeExtensionSubMenu, SubMenuItr.Value),
				bInOpenSubMenuOnClick,
				FSlateIcon(),
				bShouldCloseWindowAfterMenuSelection
			);
		}

		// end the section
		MenuBuilder.EndSection();
	}

}

void SCustomLaunchCustomProfileEditor::MakeExtensionMenu( FMenuBuilder& MenuBuilder, EMenuType MenuType, TSharedRef<ProjectLauncher::FLaunchExtension> Extension, ILauncherProfileBuildCookRunPtr BuildCookRun )
{
	TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> ExtensionInstance = ProjectLauncher::FLaunchExtension::GetExtensionInstance(Extension, CurrentProfile.ToSharedRef());

	if (MenuType == EMenuType::UATCommands && Extension->IsUATCommandManager() && Extension->CanBeCreated(CurrentProfile.ToSharedRef(), Model.ToSharedRef()) )
	{
		MenuBuilder.AddMenuEntry(
			Extension->GetDisplayName(),
			FText::GetEmpty(),
			FSlateIcon(Extension->GetIcon()),
			FUIAction(
				FExecuteAction::CreateLambda( [this, Extension]()
				{
					TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> Instance = Extension->AsUATCommandFactory()->CreateUATCommandExtensionInstance(CurrentProfile.ToSharedRef(), Model.ToSharedRef(), TreeBuilder->GetProfileTree().Get());
					if (Instance.IsValid())
					{
						TreeBuilder->GetProfileTree()->OnLaunchExtensionToggle(Instance.ToSharedRef(), true);
						PendingNewHeadingAdded = Instance->GetUATCommand()->GetInternalName();
					}
				})
			),
			NAME_None);
	}
	else if (MenuType == EMenuType::ProfileExtensions && !Extension->IsUATCommandManager() && !Extension->IsAlwaysCreated(CurrentProfile.ToSharedRef(), Model.ToSharedRef()))
	{
		auto OnToggleExtension = [this, Extension, ExtensionInstance]()
		{
			if (ExtensionInstance.IsValid())
			{
				TreeBuilder->GetProfileTree()->OnLaunchExtensionToggle(ExtensionInstance.ToSharedRef(), false);

				// defer the delete until the next tick
				PendingExtensionDeletes.AddUnique(ExtensionInstance.ToSharedRef());
			}
			else
			{
				TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> Instance = ProjectLauncher::FLaunchExtension::CreateExtensionInstance(Extension, CurrentProfile.ToSharedRef(), Model.ToSharedRef(), TreeBuilder->GetProfileTree().Get());
				if (Instance.IsValid())
				{
					TreeBuilder->GetProfileTree()->OnLaunchExtensionToggle(Instance.ToSharedRef(), true);
					TreeBuilder->OnPropertyChanged();
					CurrentProfile->RefreshCustomWarningsAndErrors();
				}
			}
				
		};

		auto CanToggleExtension = [this, Extension, ExtensionInstance]
		{
			if (ExtensionInstance.IsValid())
			{
				return ExtensionInstance->CanBeRemoved();
			}
			else
			{
				return Extension->CanBeCreated(CurrentProfile.ToSharedRef(), Model.ToSharedRef());
			}
		};

		auto GetToolTipText = [this, Extension, ExtensionInstance]
		{
			if (ExtensionInstance.IsValid() && !ExtensionInstance->CanBeRemoved())
			{
				return LOCTEXT("CannotRemoveExtensionTooltip", "The extension cannot be removed at this time");
			}
			else if (!ExtensionInstance.IsValid() && !Extension->CanBeCreated(CurrentProfile.ToSharedRef(), Model.ToSharedRef()))
			{
				return LOCTEXT("CannotAddExtensionTooltip", "The extension cannot be added at this time");
			}

			return FText::GetEmpty();
		};

		MenuBuilder.AddMenuEntry(
			Extension->GetDisplayName(),
			GetToolTipText(),
			Extension->GetIcon(),
			FUIAction(
				FExecuteAction::CreateLambda(OnToggleExtension),
				FCanExecuteAction::CreateLambda(CanToggleExtension),
				FIsActionChecked::CreateLambda( [ExtensionInstance]() { return ExtensionInstance.IsValid(); } )
			),
			NAME_None,
			EUserInterfaceActionType::Check);
	}

	if (ExtensionInstance)
	{
		if (BuildCookRun.IsValid())
		{
			if (ProjectLauncher::IBuildCookRunExtensionFactory* BuildCookRunFactory = ExtensionInstance->AsBuildCookRunFactory())
			{
				BuildCookRunFactory->CustomizeBuildCookRunExtensionMenu(MenuBuilder, BuildCookRun.ToSharedRef());
			}
		}

		if (MenuType == EMenuType::ProfileExtensions)
		{
			ExtensionInstance->MakeCustomExtensionSubmenu(MenuBuilder);
		}
	}

	if (MenuType == EMenuType::ProfileExtensions)
	{
		Extension->MakeCustomExtensionSubmenu(MenuBuilder, CurrentProfile.ToSharedRef(), Model.ToSharedRef());
	}
};






TSharedRef<SWidget> SCustomLaunchCustomProfileEditor::MakeCommandsMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	const bool bCloseSelfOnly = true;
	const bool bSearchable = false;
	const bool bInOpenSubMenuOnClick = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr, nullptr, bCloseSelfOnly, &FCoreStyle::Get(), bSearchable );

	MakeMenu( MenuBuilder, EMenuType::UATCommands );

	return MenuBuilder.MakeWidget();
}

void SCustomLaunchCustomProfileEditor::MakeUATCommandSubMenu( FMenuBuilder& MenuBuilder, ILauncherProfileUATCommandRef UATCommand )
{
	ProjectLauncher::FLaunchProfileTreeDataRef TreeData = TreeBuilder->GetProfileTree();

	const bool bShouldCloseWindowAfterMenuSelection = true;
	const bool bSearchable = false;
	const bool bInOpenSubMenuOnClick = false;

	MenuBuilder.SetSearchable(bSearchable);
	MenuBuilder.AddMenuEntry(
		LOCTEXT("EnableCommandLabel", "Enabled"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this, UATCommand]()
			{
				UATCommand->SetEnabled( !UATCommand->IsEnabled() ); 
				TreeBuilder->OnPropertyChanged();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda( [UATCommand]() { return UATCommand->IsEnabled(); } )
		),
		NAME_None,
		EUserInterfaceActionType::Check);

	if (UATCommandNameFields.Contains(UATCommand.ToSharedPtr()))
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("RenameCommandLabel", "Rename"),
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda( [this, UATCommand]()
				{
					BeginRenameUATCommand(UATCommand);
				})
			),
			NAME_None);
	}

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RemoveCommandLabel", "Delete"),
		FText::GetEmpty(),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda( [this, TreeData, UATCommand]() 
			{
				if (ProjectLauncher::GetUserConfirmation(LOCTEXT("DeleteTaskMsg","Confirm task deletion?"), LOCTEXT("DeleteTaskTitle","Delete task"), false))
				{
					// see if there is an extension associated with this UAT command
					TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> ExtensionInstance = GetExtensionForUATCommand(UATCommand);
					if (ExtensionInstance.IsValid())
					{
						TreeData->OnLaunchExtensionToggle(ExtensionInstance.ToSharedRef(), false);
					}

					TreeData->Profile->RemoveUATCommand(UATCommand->GetInternalName()); 

					if (ExtensionInstance.IsValid())
					{
						// defer the delete until the next tick
						PendingExtensionDeletes.AddUnique(ExtensionInstance.ToSharedRef());
					}
					else
					{
						// likely a BuildCookRun
						TreeBuilder->OnPropertyChanged();
						TreeData->RequestFullTreeRebuild();
					}
				}
			})
		),
		NAME_None);

	ILauncherProfileBuildCookRunPtr BuildCookRun = UATCommand->AsBuildCookRun();
	if (BuildCookRun.IsValid())
	{
		MakeMenu(MenuBuilder, EMenuType::BuildCookRunExtensions, BuildCookRun);
	}
}


void SCustomLaunchCustomProfileEditor::MakeTreeNodeSubMenu( FMenuBuilder& MenuBuilder, ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode)
{
	if (TreeNode->UATCommand.IsValid() && (!Model->CanUseSimplifiedLayout(CurrentProfile.ToSharedRef()) || UATCommandNameFields.Contains(TreeNode->UATCommand)))
	{
		MakeUATCommandSubMenu(MenuBuilder, TreeNode->UATCommand.ToSharedRef());
	}
	else
	{
		MakeMenu(MenuBuilder, EMenuType::ProfileExtensions);
	}
}

TSharedRef<SWidget> SCustomLaunchCustomProfileEditor::MakeTreeNodeSubMenuWidget(ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode)
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	const bool bCloseSelfOnly = true;
	const bool bSearchable = false;
	const bool bInOpenSubMenuOnClick = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr, nullptr, bCloseSelfOnly, &FCoreStyle::Get(), bSearchable );

	MakeTreeNodeSubMenu(MenuBuilder, TreeNode);

	return MenuBuilder.MakeWidget();
}

TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> SCustomLaunchCustomProfileEditor::GetExtensionForUATCommand(ILauncherProfileUATCommandRef UATCommand) const
{
	ProjectLauncher::FLaunchProfileTreeDataRef TreeData = TreeBuilder->GetProfileTree();

	TSharedPtr<ProjectLauncher::FLaunchExtensionInstance>* ExtensionInstancePtr = TreeData->ExtensionInstances.FindByPredicate( [UATCommand]( TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> ExtensionInstance )
	{
		return ExtensionInstance->ManagesUATCommand(UATCommand);
	});
	
	if ( ExtensionInstancePtr != nullptr)
	{
		return *ExtensionInstancePtr;
	}

	return nullptr;
}


FSlateIcon SCustomLaunchCustomProfileEditor::GetIconForUATCommand(ILauncherProfileUATCommandPtr UATCommand) const
{
	if (UATCommand.IsValid())
	{
		TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> ExtensionInstance = GetExtensionForUATCommand(UATCommand.ToSharedRef());
		if (ExtensionInstance.IsValid())
		{
			return ExtensionInstance->GetExtension()->GetIcon();
		}

		if (UATCommand->AsBuildCookRun() != nullptr)
		{
			return FSlateIcon(FProjectLauncherStyle::GetStyleSetName(), "Icons.Task.Launch");
		}
	}

	return FSlateIcon();
}


void SCustomLaunchCustomProfileEditor::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (TreeBuilder.IsValid())
	{
		// expand anything that has just been added
		if (!TreeBuilder->GetProfileTree()->RequestExpandHeadings.IsEmpty())
		{
			for (const ProjectLauncher::FLaunchProfileTreeNodePtr& Node : TreeBuilder->GetProfileTree()->RequestExpandHeadings)
			{
				TreeView->SetItemExpansion(Node, true);
			}
			TreeBuilder->GetProfileTree()->RequestExpandHeadings.Reset();
		}

		// check if there's any refreshing / rebuilding to be done
		if (TreeBuilder->GetProfileTree()->bRequestTreeFullRebuild || PendingExtensionDeletes.Num() > 0)
		{
			TreeBuilder->GetProfileTree()->bRequestTreeFullRebuild = false;

			// wait until after the slate tick before applying deletes & refreshing
			ExecuteOnGameThread(UE_SOURCE_LOCATION,[this, PendingDeletes = MoveTemp(PendingExtensionDeletes)]()
			{
				bool bProfileDirty = false;
				for (const TSharedRef<ProjectLauncher::FLaunchExtensionInstance>& ExtensionInstance : PendingDeletes)
				{
					ProjectLauncher::FLaunchExtension::DestroyExtensionInstance(ExtensionInstance, CurrentProfile.ToSharedRef(), Model.ToSharedRef());
					bProfileDirty = true;
				}

				RebuildTree();

				if (bProfileDirty)
				{
					TreeBuilder->OnPropertyChanged();
					CurrentProfile->RefreshCustomWarningsAndErrors();
				}
			});
		}
		else if (TreeBuilder->GetProfileTree()->bRequestTreeRefresh)
		{
			TreeView->RequestTreeRefresh();
			TreeBuilder->GetProfileTree()->bRequestTreeRefresh = false;
		}
		else if (!PendingNewHeadingAdded.IsEmpty())
		{
			// find the heading node
			for ( TTuple<FString,ProjectLauncher::FLaunchProfileTreeNodePtr> Pair : TreeBuilder->GetProfileTree()->HeadingNodes)
			{
				if (Pair.Key == PendingNewHeadingAdded)
				{
					TreeView->RequestScrollIntoView(Pair.Value);

					// start renaming it until after the new node has been scrolled into view
					ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, UATCommand = Pair.Value->UATCommand.ToSharedRef()]()
					{
						BeginRenameUATCommand(UATCommand);
					});

					break;
				}
			}
			PendingNewHeadingAdded.Reset();
		}
		
		if (PendingCollapseHeadings.Num() > 0)
		{
			for ( TTuple<FString,ProjectLauncher::FLaunchProfileTreeNodePtr> Pair : TreeBuilder->GetProfileTree()->HeadingNodes)
			{
				ApplyCollapsedRecusive(PendingCollapseHeadings, Pair.Key, Pair.Value);
			}

			PendingCollapseHeadings.Reset();
		}

	}
}




#undef LOCTEXT_NAMESPACE
