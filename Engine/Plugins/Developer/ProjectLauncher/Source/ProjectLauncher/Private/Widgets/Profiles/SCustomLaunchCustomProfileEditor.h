// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Framework/SlateDelegates.h"
#include "Framework/Layout/IScrollableWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Model/ProjectLauncherModel.h"
#include "Widgets/Views/STableRow.h"
#include "ProfileTree/ILaunchProfileTreeBuilder.h"

struct FSlateIcon;
class SWidget;
class ITableRow;
class STableViewBase;
template<typename T> class STreeView;

namespace ProjectLauncher
{
	class FLaunchExtension;
	class FLaunchExtensionInstance;
};
typedef STreeView<ProjectLauncher::FLaunchProfileTreeNodePtr> SLaunchProfileTreeView;
typedef STableRow<ProjectLauncher::FLaunchProfileTreeNodePtr> SLaunchProfileTreeRow;


class SCustomLaunchCustomProfileEditor : public SCompoundWidget, public IScrollableWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomLaunchCustomProfileEditor) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel);

	void SetProfile( const ILauncherProfilePtr& Profile );
	ILauncherProfilePtr GetProfile() const { return CurrentProfile; }

	TSharedRef<SWidget> MakeCommandsMenu();

	void RebuildTree();
	TSharedRef<SLaunchProfileTreeView> GetTreeView() const { return TreeView.ToSharedRef(); }

	// IScrollableWidget interface
	virtual FVector2D GetScrollDistance() override;
	virtual FVector2D GetScrollDistanceRemaining() override;
	virtual TSharedRef<SWidget> GetScrollWidget() override;

private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedRef<ITableRow> OnGenerateWidgetForTreeNode( ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode, const TSharedRef<STableViewBase>& OwnerTable, bool bPinned );
	void OnGetChildren(ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode, TArray<ProjectLauncher::FLaunchProfileTreeNodePtr>& OutChildren);

	void CacheCollapsedRecursive( TSet<FString>& Collapsed, const FString& NodeName, ProjectLauncher::FLaunchProfileTreeNodePtr Node) const;
	void ApplyCollapsedRecusive( const TSet<FString>& Collapsed, const FString& NodeName, ProjectLauncher::FLaunchProfileTreeNodePtr Node) const;
	void ReorderNodes(ProjectLauncher::FLaunchProfileTreeNodePtr SourceNode, ProjectLauncher::FLaunchProfileTreeNodePtr TargetNode, EItemDropZone DropZone);

	bool IsHierarchyVisible(ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode) const;

	enum class EMenuType
	{
		ProfileExtensions,
		BuildCookRunExtensions,
		UATCommands,
	};
	void MakeMenu( FMenuBuilder& MenuBuilder, EMenuType MenuType, ILauncherProfileBuildCookRunPtr BuildCookRun = nullptr );

	void MakeUATCommandSubMenu( FMenuBuilder& MenuBuilder, ILauncherProfileUATCommandRef UATCommand );
	void MakeExtensionMenu( FMenuBuilder& MenuBuilder, EMenuType MenuType, TSharedRef<ProjectLauncher::FLaunchExtension> Extension, ILauncherProfileBuildCookRunPtr BuildCookRun = nullptr );

	void MakeTreeNodeSubMenu( FMenuBuilder& MenuBuilder, ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode);
	TSharedRef<SWidget> MakeTreeNodeSubMenuWidget(ProjectLauncher::FLaunchProfileTreeNodePtr TreeNode);

	TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> GetExtensionForUATCommand(ILauncherProfileUATCommandRef UATCommand) const;
	FSlateIcon GetIconForUATCommand(ILauncherProfileUATCommandPtr UATCommand) const;

	ILauncherProfilePtr CurrentProfile;
	TSharedPtr<ProjectLauncher::FModel> Model;
	TSharedPtr<ProjectLauncher::ILaunchProfileTreeBuilder> TreeBuilder;
	TSharedPtr<SLaunchProfileTreeView> TreeView;
	TArray<TSharedRef<ProjectLauncher::FLaunchExtensionInstance>> PendingExtensionDeletes;
	TSet<FString> PendingCollapseHeadings;
	FString PendingNewHeadingAdded;

	float SplitterPos = 0.6f;
	

	void BeginRenameUATCommand( const ILauncherProfileUATCommandRef& UATCommand ) const;

private:
	TMap<ILauncherProfilePtr,TSet<FString>> CachedCollapsedNodes;
	TMap<ILauncherProfileUATCommandPtr,TWeakPtr<class SInlineEditableTextBlock>> UATCommandNameFields;
	friend class SLaunchProfileCategoryTreeRow;
};
