// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Profiles/SCustomLaunchCustomProfileSelector.h"

#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/ProjectLauncherStyle.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Shared/SCustomLaunchDeviceCombo.h"
#include "SPositiveActionButton.h"
#include "DesktopPlatformModule.h"
#include "PlatformInfo.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "InstalledPlatformInfo.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/ConfigCacheIni.h"
#include "GameProjectHelper.h"


#define LOCTEXT_NAMESPACE "SCustomLaunchCustomProfileSelector"


SCustomLaunchCustomProfileSelector::~SCustomLaunchCustomProfileSelector()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);

		Model->GetProfileManager()->OnProfileAdded().RemoveAll(this);
		Model->GetProfileManager()->OnProfileRemoved().RemoveAll(this);

		const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager = Model->GetDeviceProxyManager();
		DeviceProxyManager->OnProxyAdded().RemoveAll(this);
		DeviceProxyManager->OnProxyRemoved().RemoveAll(this);
	}
}



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchCustomProfileSelector::Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel)
{
	Model = InModel;
	OnProfileAdd = InArgs._OnProfileAdd;
	OnProfileDelete = InArgs._OnProfileDelete;
	OnProfileDuplicate = InArgs._OnProfileDuplicate;
	OnProfileEdit = InArgs._OnProfileEdit;
	OnProfileRename = InArgs._OnProfileRename;
	OnProfileEditDescription = InArgs._OnProfileEditDescription;
	OnProfileModified = InArgs._OnProfileModified;
	ChangeProfileEditorVisibility = InArgs._ChangeProfileEditorVisibility;
	EditPanelVisible = InArgs._EditPanelVisible;

	CustomProfileListView = SNew(SListView<ILauncherProfilePtr>)
	.ListItemsSource(&Model->GetAllProfiles())
	.ClearSelectionOnClick(false)
	.SelectionMode(ESelectionMode::Single)
	.OnMouseButtonDoubleClick(this, &SCustomLaunchCustomProfileSelector::EditProfile)
	.OnSelectionChanged(this, &SCustomLaunchCustomProfileSelector::OnSelectionChanged)
	.OnGenerateRow(this, &SCustomLaunchCustomProfileSelector::GenerateCustomProfileRow)
	.OnContextMenuOpening(this, &SCustomLaunchCustomProfileSelector::MakeContextMenu)
	;

	ChildSlot
	[
		// main content
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew(SScrollBorder, CustomProfileListView.ToSharedRef())
			[
				CustomProfileListView.ToSharedRef()
			]
		]
	];

	// hook profile changes
	Model->GetProfileManager()->OnProfileAdded().AddSP(this, &SCustomLaunchCustomProfileSelector::OnCustomProfileAdded);
	Model->GetProfileManager()->OnProfileRemoved().AddSP(this, &SCustomLaunchCustomProfileSelector::OnCustomProfileRemoved);

	Model->OnProfileSelected().AddSP(this, &SCustomLaunchCustomProfileSelector::OnProfileSelected);
	OnProfileSelected(Model->GetSelectedProfile(), nullptr);
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION











BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<ITableRow> SCustomLaunchCustomProfileSelector::GenerateCustomProfileRow(ILauncherProfilePtr Profile, const TSharedRef<STableViewBase>& OwnerTable)
{
	bool bIsBasicLaunchProfile = Model->IsBasicLaunchProfile(Profile);

	auto GetErrorVisibility = [Profile]()
	{
		if (Profile->HasValidationError())
		{
			return EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	};

	auto GetErrorToolTipText = [Profile]()
	{
		return ProjectLauncher::GetProfileLaunchErrorMessage(Profile);
	};


	TSharedPtr<SVerticalBox> DetailsBox;
	TSharedPtr<SVerticalBox> ButtonsBox;
	TSharedPtr<SHorizontalBox> MainBox;
	TSharedPtr<SBorder> CustomSelectionBorder;
	TSharedRef<SLauncherTableRow> RowContent = SNew(SLauncherTableRow, OwnerTable)
	.ShowSelection(false)
	[
		SNew(SBorder)
		.Padding(FMargin(4,4,4,0))
		.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
		[
			SAssignNew(CustomSelectionBorder, SBorder)
			.BorderImage(FProjectLauncherStyle::GetBrush("WhiteGroupBorder"))
			[
				SAssignNew(MainBox, SHorizontalBox)

				// platform icon
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(48,48))
					.Image(this, &SCustomLaunchCustomProfileSelector::GetProfileImage, Profile)
				]

				// name and description etc.
				+SHorizontalBox::Slot()
				.FillWidth(1)
				.Padding(4)
				[
					SAssignNew(DetailsBox, SVerticalBox)
				]
			]
		]
	];

	// set the color attribute after the row itself has been created - this allows us to capture the row widget itself
	CustomSelectionBorder->SetBorderBackgroundColor(TAttribute<FSlateColor>::CreateSP(this, &SCustomLaunchCustomProfileSelector::GetRowColor, RowContent) );
	
	TSharedPtr<SHorizontalBox> NameBox;


	if (bIsBasicLaunchProfile)
	{
		// add the name
		DetailsBox->AddSlot()
		.Padding(4)
		.AutoHeight()
		[
			SAssignNew(NameBox, SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.ColorAndOpacity(FColor::White)
				.Text_Lambda( [Profile]() { return FText::FromString(Profile->GetName()); })
			]
		];

		// add the quick device picker
		DetailsBox->AddSlot()
		.Padding(4)
		.AutoHeight()
		[
			SNew(SHorizontalBox)

			// device picker
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCustomLaunchDeviceCombo)
				.OnSelectionChanged(this, &SCustomLaunchCustomProfileSelector::SetSelectedDevices, Profile)
				.SelectedDevices(this, &SCustomLaunchCustomProfileSelector::GetSelectedDevices, Profile)
				.AllPlatforms(true)
			]

			// device manager button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4,0)
			[
				SNew(SButton)
				.OnClicked(this, &SCustomLaunchCustomProfileSelector::OnOpenDeviceManagerClicked)
				.ButtonStyle(FProjectLauncherStyle::Get(), "HoverHintOnly")
				.ToolTipText(LOCTEXT("OpenDeviceManagerToolTip", "Open the Device Manager window, where you can setup and claim devices connected to your machine or shared on the network."))
				[
					SNew(SImage)
					.Image(FProjectLauncherStyle::Get().GetBrush("Icons.DeviceManager"))
					.ColorAndOpacity(FColor::White)
				]
			]
		];

		// fixme: current settings etc.?
	}
	else
	{
		TSharedPtr<SInlineEditableTextBlock> NameEditTextBox;
		TSharedPtr<SInlineEditableTextBlock> DescriptionEditTextBox;

		// name edit box
		DetailsBox->AddSlot()
		.Padding(4)
		.AutoHeight()
		[
			SAssignNew(NameBox, SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SAssignNew(NameEditTextBox, SInlineEditableTextBlock)
				.Text_Lambda( [Profile]() { return FText::FromString(Profile->GetName()); })
				.ColorAndOpacity(FColor::White)
				.OnTextCommitted( this, &SCustomLaunchCustomProfileSelector::SetProfileName, Profile )
			]
		];

		// description
		DetailsBox->AddSlot()
		.Padding(4)
		.AutoHeight()
		[
			SAssignNew(DescriptionEditTextBox, SInlineEditableTextBlock)
			.Text_Lambda( [Profile]() { return FText::FromString(Profile->GetDescription()); })
			.OnTextCommitted( this, &SCustomLaunchCustomProfileSelector::SetProfileDescription, Profile )
			.OnExitEditingMode_Lambda( [this]() { Model->SortProfiles(); RefreshCustomProfileList(); } )
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.HintText(LOCTEXT("DescriptionHintText", "Enter a description for this profile"))
		];

		NameEditTextBoxes.Add(Profile, NameEditTextBox);
		DescriptionEditTextBoxes.Add(Profile, DescriptionEditTextBox);
	}

	// validation warning icon
	NameBox->AddSlot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(4, 0)
	.AutoWidth()
	[
		SNew(SImage)
		.Image(FProjectLauncherStyle::Get().GetBrush("Icons.WarningWithColor.Small"))
		.Visibility_Lambda(GetErrorVisibility)
		.ToolTipText_Lambda(GetErrorToolTipText)
	];



	// floating edit button
	MainBox->AddSlot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Right)
	.Padding(8)
	.AutoWidth()
	[
		SNew(SButton)
		.OnClicked(this, &SCustomLaunchCustomProfileSelector::OnEditProfileClicked, Profile)
		.ButtonStyle(FProjectLauncherStyle::Get(), "HoverHintOnly")
		.Visibility( this, &SCustomLaunchCustomProfileSelector::GetInlineEditButtonVisibility, RowContent->AsWidget(), Profile )
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Edit"))
		]
	];

	// close editor button
	MainBox->AddSlot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Right)
	.Padding(8)
	.AutoWidth()
	[
		SNew(SButton)
		.OnClicked(this, &SCustomLaunchCustomProfileSelector::OnCloseEditorClicked)
		.ButtonStyle(FProjectLauncherStyle::Get(), "HoverHintOnly")
		.Visibility( this, &SCustomLaunchCustomProfileSelector::GetCloseEditorButtonVisibility, Profile)
		[
			SNew(SImage)
			.Image(FProjectLauncherStyle::Get().GetBrush("SidePanelRightClose"))
		]
	];

	return RowContent;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

const FSlateBrush* SCustomLaunchCustomProfileSelector::GetProfileImage(ILauncherProfilePtr Profile) const
{
	if (Profile.IsValid() && Model->IsAdvancedProfile(Profile.ToSharedRef()))
	{
		return FAppStyle::GetBrush("Icons.Warning.Large");
	}
	else
	{
		TArray<const PlatformInfo::FTargetPlatformInfo*> PlatformInfos = Model->GetPlatformInfosForAllCommands(Profile);
		return FProjectLauncherStyle::GetProfileBrushForPlatforms(PlatformInfos, EPlatformIconSize::Large);
	}
}

FSlateColor SCustomLaunchCustomProfileSelector::GetRowColor(TSharedRef<SLauncherTableRow> TableRow) const
{
	if (TableRow->IsItemSelected())
	{
#if 1	// always maintain active state, as per UX design
		return FStyleColors::Select;
#else
		if (CustomProfileListView->HasKeyboardFocus())
		{
			return FStyleColors::Select;
		}
		else
		{
			return FStyleColors::SelectInactive;
		}
#endif
	}
	else
	{
		if (TableRow->IsHovered())
		{
			return FStyleColors::Hover;
		}
		else
		{
			return FStyleColors::Panel;
		}
	}
}

void SCustomLaunchCustomProfileSelector::OnSelectionChanged( ILauncherProfilePtr Profile, ESelectInfo::Type SelectionMode )
{
	if (Profile.IsValid() && ensure(CustomProfileListView->IsItemSelected(Profile)))
	{
		Model->SelectProfile(Profile);
	}
}

void SCustomLaunchCustomProfileSelector::OnProfileSelected( const ILauncherProfilePtr& NewProfile, const ILauncherProfilePtr& OldProfile)
{
	if (OldProfile.IsValid() && CustomProfileListView->IsItemSelected(OldProfile))
	{
		CustomProfileListView->SetItemSelection(OldProfile, false, ESelectInfo::Direct);
	}

	if (NewProfile.IsValid())
	{
		if (!CustomProfileListView->IsItemSelected(NewProfile))
		{
			CustomProfileListView->SetItemSelection(NewProfile, true, ESelectInfo::Direct);
			CustomProfileListView->RequestScrollIntoView(NewProfile);
		}
	}
}


void SCustomLaunchCustomProfileSelector::OnCustomProfileAdded(const ILauncherProfileRef& AddedProfile)
{
	RefreshCustomProfileList();
}



void SCustomLaunchCustomProfileSelector::OnCustomProfileRemoved(const ILauncherProfileRef& RemovedProfile)
{
	NameEditTextBoxes.Remove(RemovedProfile.ToSharedPtr());
	DescriptionEditTextBoxes.Remove(RemovedProfile.ToSharedPtr());

	RefreshCustomProfileList();
}



void SCustomLaunchCustomProfileSelector::RefreshCustomProfileList()
{
	CustomProfileListView->RequestListRefresh();
}


TSharedPtr<SWidget> SCustomLaunchCustomProfileSelector::MakeContextMenu()
{
	ILauncherProfilePtr Profile = Model->GetSelectedProfile();
	bool bIsBasicLaunchProfile = Model->IsBasicLaunchProfile(Profile);

	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CustomProfileAddLabel", "Add Profile"),
		LOCTEXT("CustomProfileAddToolTip", "Add a new profile"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Plus"),
		FUIAction(
			FExecuteAction::CreateLambda( [this]() { OnProfileAdd.Execute(); } )
		),
		NAME_None);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CustomProfileEditLabel", "Edit Profile"),
		LOCTEXT("CustomProfileEditToolTip", "Edit this profile"),
		FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Edit"),
		FUIAction(
			FExecuteAction::CreateSP(this, &SCustomLaunchCustomProfileSelector::OnCustomProfileEditClicked, Profile),
			FCanExecuteAction(),
			FGetActionCheckState::CreateSP(this, &SCustomLaunchCustomProfileSelector::GetCustomProfileEditCheckState, Profile)
		),
		NAME_None,
		EUserInterfaceActionType::Check);

	if (!bIsBasicLaunchProfile)
	{
		MenuBuilder.AddMenuSeparator();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CustomProfileRenameLabel", "Rename Profile"),
			LOCTEXT("CustomProfileRenameToolTip", "Rename this profile"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Edit"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SCustomLaunchCustomProfileSelector::OnCustomProfileRenameClicked, Profile)
			),
			NAME_None);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CustomProfileEditDescLabel", "Edit Description"),
			LOCTEXT("CustomProfileEditDescToolTip", "Edit the description of this profile"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Edit"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SCustomLaunchCustomProfileSelector::OnCustomProfileEditDescriptionClicked, Profile)
			),
			NAME_None);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CustomProfileDuplicateLabel", "Duplicate Profile"),
			LOCTEXT("CustomProfileDuplicateToolTip", "Duplicate this profile"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Duplicate"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SCustomLaunchCustomProfileSelector::OnCustomProfileDuplicateClicked, Profile)
			),
			NAME_None);

		MenuBuilder.AddMenuSeparator();

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CustomProfileDeleteLabel", "Delete Profile"),
			LOCTEXT("CustomProfileDeleteToolTip", "Deletes this profile"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Delete"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SCustomLaunchCustomProfileSelector::OnCustomProfileDeleteClicked, Profile)
			),
			NAME_None);
		
	}

	return MenuBuilder.MakeWidget();
}

void SCustomLaunchCustomProfileSelector::OnCustomProfileEditClicked(ILauncherProfilePtr Profile)
{
	if ( EditPanelVisible.Get() && Model->GetSelectedProfile() == Profile)
	{
		ChangeProfileEditorVisibility.ExecuteIfBound(false);
	}
	else
	{
		EditProfile(Profile);
	}
}

void SCustomLaunchCustomProfileSelector::OnCustomProfileDuplicateClicked(ILauncherProfilePtr Profile)
{
	OnProfileDuplicate.ExecuteIfBound(Profile);
}

void SCustomLaunchCustomProfileSelector::OnCustomProfileDeleteClicked(ILauncherProfilePtr Profile)
{
	OnProfileDelete.ExecuteIfBound(Profile);
}

void SCustomLaunchCustomProfileSelector::OnCustomProfileRenameClicked(ILauncherProfilePtr Profile)
{
	OnProfileRename.ExecuteIfBound(Profile);
}

void SCustomLaunchCustomProfileSelector::OnCustomProfileEditDescriptionClicked(ILauncherProfilePtr Profile)
{
	OnProfileEditDescription.ExecuteIfBound(Profile);
}

void SCustomLaunchCustomProfileSelector::SetSelectedDevices( const TArray<FString> DeviceIDs, ILauncherProfilePtr Profile)
{
	check(Model->IsBasicLaunchProfile(Profile));

	ILauncherProfileBuildCookRunPtr BuildCookRun = Profile->GetFirstBuildCookRun();
	check(BuildCookRun.IsValid());

	if (ensure(BuildCookRun->GetDeployedDeviceGroup() != nullptr))
	{
		BuildCookRun->GetDeployedDeviceGroup()->RemoveAllDevices();
		for (const FString& DeviceID : DeviceIDs)
		{
			BuildCookRun->GetDeployedDeviceGroup()->AddDevice(DeviceID);
		}
		Model->UpdatedCookedPlatformsFromDeployDeviceProxy(Profile.ToSharedRef(), BuildCookRun.ToSharedRef());

		OnProfileModified.ExecuteIfBound(Profile);
	}
}

TArray<FString> SCustomLaunchCustomProfileSelector::GetSelectedDevices(const ILauncherProfilePtr Profile) const
{
	check(Model->IsBasicLaunchProfile(Profile));

	ILauncherProfileBuildCookRunPtr BuildCookRun = Profile->GetFirstBuildCookRun();
	check(BuildCookRun.IsValid());

	if (ensure(BuildCookRun->GetDeployedDeviceGroup() != nullptr))
	{
		return BuildCookRun->GetDeployedDeviceGroup()->GetDeviceIDs();
	}

	return TArray<FString>();
}

FReply SCustomLaunchCustomProfileSelector::OnOpenDeviceManagerClicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId("DeviceManager"));
	return FReply::Handled();
}

FReply SCustomLaunchCustomProfileSelector::OnEditProfileClicked(ILauncherProfilePtr Profile)
{
	EditProfile(Profile);

	return FReply::Handled();
}

FReply SCustomLaunchCustomProfileSelector::OnCloseEditorClicked()
{
	ChangeProfileEditorVisibility.ExecuteIfBound(false);

	return FReply::Handled();
}

EVisibility SCustomLaunchCustomProfileSelector::GetCloseEditorButtonVisibility(ILauncherProfilePtr Profile) const
{
	if ( EditPanelVisible.Get() && Model->GetSelectedProfile() == Profile)
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed;
	}
}

ECheckBoxState SCustomLaunchCustomProfileSelector::GetCustomProfileEditCheckState(ILauncherProfilePtr Profile) const
{
	if ( EditPanelVisible.Get() && Model->GetSelectedProfile() == Profile)
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}


void SCustomLaunchCustomProfileSelector::EditProfile(ILauncherProfilePtr Profile)
{
	if ( EditPanelVisible.Get() && Model->GetSelectedProfile() == Profile)
	{
		ChangeProfileEditorVisibility.ExecuteIfBound(false);
	}
	else
	{
		OnProfileEdit.ExecuteIfBound(Profile);
	}
}

EVisibility SCustomLaunchCustomProfileSelector::GetInlineEditButtonVisibility(TSharedRef<SWidget> RowWidget, ILauncherProfilePtr Profile) const
{
	if (EditPanelVisible.Get())
	{
		return EVisibility::Collapsed;
	}
	else if (RowWidget->IsHovered())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Hidden;
	}
}


void SCustomLaunchCustomProfileSelector::StartEditProfileName(ILauncherProfilePtr Profile)
{
	if (Profile.IsValid() && NameEditTextBoxes.Contains(Profile))
	{
		NameEditTextBoxes.FindRef(Profile)->EnterEditingMode();
	}
}

void SCustomLaunchCustomProfileSelector::StartEditProfileDescription(ILauncherProfilePtr Profile)
{
	if (Profile.IsValid() && DescriptionEditTextBoxes.Contains(Profile))
	{
		DescriptionEditTextBoxes.FindRef(Profile)->EnterEditingMode();
	}
}


void SCustomLaunchCustomProfileSelector::SetProfileName(const FText& NewText, ETextCommit::Type InTextCommit, ILauncherProfilePtr Profile)
{
	if (Profile.IsValid())
	{
		Model->GetProfileManager()->ChangeProfileName(Profile.ToSharedRef(), NewText.ToString());

		if (InTextCommit == ETextCommit::OnEnter || InTextCommit == ETextCommit::OnUserMovedFocus)
		{
			StartEditProfileDescription(Profile);
		}
		else
		{
			Model->SortProfiles();
			RefreshCustomProfileList();
		}
	}
}

void SCustomLaunchCustomProfileSelector::SetProfileDescription(const FText& NewText, ETextCommit::Type InTextCommit, ILauncherProfilePtr Profile)
{
	if (Profile.IsValid())
	{
		Profile->SetDescription(NewText.ToString());
		Model->GetProfileManager()->SaveJSONProfile(Profile.ToSharedRef());	
	}
}





#undef LOCTEXT_NAMESPACE
