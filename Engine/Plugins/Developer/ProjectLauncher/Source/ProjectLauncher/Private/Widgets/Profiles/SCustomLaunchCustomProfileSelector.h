// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Model/ProjectLauncherModel.h"

class SWidget;
class ITableRow;
class STableViewBase;
class SInlineEditableTextBlock;
template<typename T> class SListView;


class SCustomLaunchCustomProfileSelector 
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FSetProfileEditorVisible, bool)

	SLATE_BEGIN_ARGS(SCustomLaunchCustomProfileSelector) {}
		SLATE_EVENT(FOnClicked, OnProfileAdd)
		SLATE_EVENT(ProjectLauncher::FOnProfileClicked, OnProfileEdit)
		SLATE_EVENT(ProjectLauncher::FOnProfileClicked, OnProfileDuplicate)
		SLATE_EVENT(ProjectLauncher::FOnProfileClicked, OnProfileDelete)
		SLATE_EVENT(ProjectLauncher::FOnProfileClicked, OnProfileRename)
		SLATE_EVENT(ProjectLauncher::FOnProfileClicked, OnProfileEditDescription)
		SLATE_EVENT(ProjectLauncher::FOnProfileClicked, OnProfileModified)
		SLATE_EVENT(FSetProfileEditorVisible, ChangeProfileEditorVisibility)
		SLATE_ATTRIBUTE(bool, EditPanelVisible)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel);
	~SCustomLaunchCustomProfileSelector();

	void StartEditProfileName(ILauncherProfilePtr Profile);
	void StartEditProfileDescription(ILauncherProfilePtr Profile);

private:
	typedef STableRow<TSharedPtr<ILauncherProfilePtr>> SLauncherTableRow;

	FOnClicked OnProfileAdd;
	ProjectLauncher::FOnProfileClicked OnProfileEdit;
	ProjectLauncher::FOnProfileClicked OnProfileDelete;
	ProjectLauncher::FOnProfileClicked OnProfileDuplicate;
	ProjectLauncher::FOnProfileClicked OnProfileRename;
	ProjectLauncher::FOnProfileClicked OnProfileEditDescription;
	ProjectLauncher::FOnProfileClicked OnProfileModified;
	FSetProfileEditorVisible ChangeProfileEditorVisibility;
	TAttribute<bool> EditPanelVisible;

	TSharedRef<ITableRow> GenerateCustomProfileRow(ILauncherProfilePtr Profile, const TSharedRef<STableViewBase>& OwnerTable);
	const FSlateBrush* GetProfileImage(ILauncherProfilePtr Profile) const;
	FSlateColor GetRowColor(TSharedRef<SLauncherTableRow> TableRow) const;

	void OnSelectionChanged( ILauncherProfilePtr Profile, ESelectInfo::Type SelectionMode );
	void OnProfileSelected( const ILauncherProfilePtr& NewProfile, const ILauncherProfilePtr& OldProfile);
	void OnCustomProfileAdded(const ILauncherProfileRef& AddedProfile);
	void OnCustomProfileRemoved(const ILauncherProfileRef& RemovedProfile);
	void OnCustomProfileEditClicked(ILauncherProfilePtr Profile);
	void OnCustomProfileDuplicateClicked(ILauncherProfilePtr Profile);
	void OnCustomProfileDeleteClicked(ILauncherProfilePtr Profile);
	void OnCustomProfileRenameClicked(ILauncherProfilePtr Profile);
	void OnCustomProfileEditDescriptionClicked(ILauncherProfilePtr Profile);
	FReply OnOpenDeviceManagerClicked();
	FReply OnEditProfileClicked(ILauncherProfilePtr Profile);
	FReply OnCloseEditorClicked();
	EVisibility GetInlineEditButtonVisibility(TSharedRef<SWidget> RowWidget, ILauncherProfilePtr Profile) const;
	EVisibility GetCloseEditorButtonVisibility(ILauncherProfilePtr Profile) const;
	ECheckBoxState GetCustomProfileEditCheckState(ILauncherProfilePtr Profile) const;

	void RefreshCustomProfileList();

	void SetSelectedDevices(const TArray<FString> DeviceIDs, ILauncherProfilePtr Profile );
	TArray<FString> GetSelectedDevices(const ILauncherProfilePtr Profile) const;

	TSharedPtr<SWidget> MakeContextMenu();
	void EditProfile(ILauncherProfilePtr Profile);

	void SetProfileName(const FText& NewText, ETextCommit::Type InTextCommit, ILauncherProfilePtr Profile);
	void SetProfileDescription(const FText& NewText, ETextCommit::Type InTextCommit, ILauncherProfilePtr Profile);

	TSharedPtr<SListView<ILauncherProfilePtr>> CustomProfileListView;

	TMap<ILauncherProfilePtr, TSharedPtr<SInlineEditableTextBlock>> NameEditTextBoxes;
	TMap<ILauncherProfilePtr, TSharedPtr<SInlineEditableTextBlock>> DescriptionEditTextBoxes;

private:
	TSharedPtr<ProjectLauncher::FModel> Model;
};
