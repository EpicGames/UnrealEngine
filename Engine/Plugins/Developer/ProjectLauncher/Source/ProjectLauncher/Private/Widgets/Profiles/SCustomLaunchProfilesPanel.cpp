// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Profiles/SCustomLaunchProfilesPanel.h"
#include "SlateOptMacros.h"
#include "Styling/StyleColors.h"
#include "Styling/ProjectLauncherStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Shared/SCustomLaunchProjectCombo.h"
#include "Widgets/Shared/SCustomLaunchBuildTargetCombo.h"
#include "Widgets/Profiles/SCustomLaunchCustomProfileSelector.h"
#include "Widgets/Profiles/SCustomLaunchCustomProfileEditor.h"
#include "Widgets/Output/SCustomLaunchLaunchPanel.h"
#include "Widgets/Output/SCustomLaunchOutputLog.h"
#include "ILauncher.h"
#include "ITargetDeviceProxyManager.h"
#include "ITargetDeviceProxy.h"
#include "Modules/ModuleManager.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PlatformInfo.h"
#include "Misc/ConfigCacheIni.h"


#define LOCTEXT_NAMESPACE "SCustomLaunchProfilesPanel"

namespace
{
	const TCHAR* WasEditorOpenKey = TEXT("WasEditorOpen");
	bool bIsEditorOpen = false;
}


SCustomLaunchProfilesPanel::SCustomLaunchProfilesPanel()
{
}


SCustomLaunchProfilesPanel::~SCustomLaunchProfilesPanel()
{
	if (Model.IsValid())
	{
		Model->OnProfileSelected().RemoveAll(this);
	}
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchProfilesPanel::Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel)
{
	Model = InModel;
	OnProfileLaunchClicked = InArgs._OnProfileLaunchClicked;

	TSharedPtr<ProjectLauncher::FLaunchLogTextLayoutMarshaller> LaunchLogTextMarshaller = MakeShared<ProjectLauncher::FLaunchLogTextLayoutMarshaller>(InModel);
	OutputLog = SNew(SCustomLaunchOutputLog, InModel, LaunchLogTextMarshaller.ToSharedRef());
	ChildSlot
	[
		SAssignNew(LogAreaSplitter, SSplitter)
		.Orientation(Orient_Vertical)
		.PhysicalSplitterHandleSize(8)
		.Style(FAppStyle::Get(), "SplitterPanel")

		// profile editor
		+SSplitter::Slot()
		.Resizable(false)
		[
			CreateProfilesPanel()
		]

		// bottom-docked log, for referencing the most recently completed build
		+SSplitter::Slot()
		.SizeRule(SSplitter::ESizeRule::SizeToContent)
		.Resizable(false)
		[
			SAssignNew(LogExpandableArea, SExpandableArea)
			.InitiallyCollapsed(true)
			.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
			.OnAreaExpansionChanged(this, &SCustomLaunchProfilesPanel::OnLogAreaExpansionChanged)
			.Padding(FMargin(0,8,0,0))
			.HeaderContent()
			[
				SNew(SHorizontalBox)

				// output log title
				+SHorizontalBox::Slot()
				.FillWidth(1)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(2)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("OutputLogAreaTitle", "Output Log"))
				]

				// output log filter box
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					OutputLog->CreateFilterWidget()
				]
			]
			.BodyContent()
			[
				OutputLog.ToSharedRef()
			]
		]
	];

	// hide profile editor by default
	bIsEditorOpen = GConfig->GetBoolOrDefault(Model->GetConfigSection(), WasEditorOpenKey, false, Model->GetConfigIni());
	SetProfileEditorVisible(bIsEditorOpen);


	Model->OnProfileSelected().AddSP(this, &SCustomLaunchProfilesPanel::OnProfileSelected);

	// wait one tick then select the last used profile or in case it's missing the Basic Launch Profile. this should give enough time for the host device proxy to be created
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]()
	{
		ILauncherProfilePtr SelectedProfile = Model->GetSelectedProfile();
		if (!SelectedProfile.IsValid())
		{
			Model->SelectProfile(Model->GetBasicLaunchProfile());
		}
		else
		{
			PropertyEditor->SetProfile(SelectedProfile);
		}
	});
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchProfilesPanel::CreateProfilesPanel()
{
	TSharedPtr<SVerticalBox> ProfilesPanel = SNew(SVerticalBox);

	// global project & target configuration (for UnrealFrontend only - not used in editor)
	if (!GIsEditor)
	{
		ProfilesPanel->AddSlot()
		.AutoHeight()
		.Padding(8,8,8,0)
		[
			SNew(SHorizontalBox)

			// Default Project selector label
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4,0)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DefaultProjectSelectLabel", "Default Project"))
			]

			// Default Project selector
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2,0)
			[
				SNew(SCustomLaunchProjectCombo)
				.OnSelectionChanged(this, &SCustomLaunchProfilesPanel::SetDefaultProjectPath)
				.SelectedProject(this, &SCustomLaunchProfilesPanel::GetDefaultProjectPath)
				.ShowAnyProjectOption(false)
			]


			// Default Build Target selector
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2,0)
			[
				SNew(SCustomLaunchBuildTargetCombo, Model.ToSharedRef())
				.OnSelectionChanged(this, &SCustomLaunchProfilesPanel::SetDefaultBuildTarget)
				.SelectedBuildTargets(this, &SCustomLaunchProfilesPanel::GetDefaultBuildTarget)
				.SelectedProject(this, &SCustomLaunchProfilesPanel::GetDefaultProjectPath)
				.IsEnabled_Lambda( [this]() { return !Model->GetSelectedProfile().IsValid() || Model->AreProjectSettingsReady(Model->GetSelectedProfile().ToSharedRef() ); } )
			]

			// Spacing
			+SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SSpacer)
			]

			// Link to legacy Project Launcher
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4,0)
			[
				SNew(SHyperlink)
				.OnNavigate_Lambda([]() { FGlobalTabmanager::Get()->TryInvokeTab(FTabId("LegacyProjectLauncher")); } )
				.Text(LOCTEXT("OpenLegacyProjectLauncher","Open Legacy Project Launcher (deprecated)"))
				.ToolTipText(LOCTEXT("OpenLegacyProjectLauncherTip", "Opens the legacy Project Launcher tab (note: this tool is deprecated)"))
			]
		];
	}

	// main profile banner
	ProfilesPanel->AddSlot()
	.AutoHeight()
	.Padding(8,8,8,0)
	[
		CreateProfileEditorToolbarWidget()
	];

	// main profile selection & editing panel
	ProfilesPanel->AddSlot()
	.FillHeight(1)
	.Padding(8,8,8,0)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ChildWindow.Background"))
		.Padding(0)
		[
			SAssignNew(ProfileEditorSplitter, SSplitter)
			.Orientation(Orient_Horizontal)
			.PhysicalSplitterHandleSize(8)
			.Style(FAppStyle::Get(), "SplitterPanel")
			
			// current profile selection panel
			+SSplitter::Slot()
			[
				CreateProfileSelectorWidget()
			]
		
			// Profile editor panel
			+SSplitter::Slot()
			[
				CreateProfileEditorWidget()
			]
		]
	];

	return ProfilesPanel.ToSharedRef();

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchProfilesPanel::CreateProfileSelectorWidget()
{
	return SNew(SVerticalBox)

	// top banner
	+SVerticalBox::Slot()
	.AutoHeight()
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
		[
			SNew(SHorizontalBox)

			// bannel label
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			.Padding(6,2)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ProfilesLabel", "Launch Profiles"))
			]

			// new profile button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(4,2)
			[
				SNew(SButton)
				.OnClicked(this, &SCustomLaunchProfilesPanel::OnCreateNewCustomProfileClicked)
				.ToolTipText(LOCTEXT("NewProfileToolTip", "Create a new custom profile"))
				[
					SNew(SHorizontalBox)
				
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
						.ColorAndOpacity(FStyleColors::AccentGreen)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CustomizeProfileLabel", "Create Launch Profile"))
					]
				]
			]
		]
	]

	// profiles list
	+SVerticalBox::Slot()
	.FillHeight(1)
	[
		SAssignNew(ProfileSelector, SCustomLaunchCustomProfileSelector, Model.ToSharedRef())
		.OnProfileAdd(this, &SCustomLaunchProfilesPanel::OnCreateNewCustomProfileClicked)
		.OnProfileDelete(this, &SCustomLaunchProfilesPanel::OnProfileDelete)
		.OnProfileDuplicate(this, &SCustomLaunchProfilesPanel::OnProfileDuplicate)
		.OnProfileEdit(this, &SCustomLaunchProfilesPanel::OnProfileEdit)
		.OnProfileRename(this, &SCustomLaunchProfilesPanel::OnProfileRename)
		.OnProfileEditDescription(this, &SCustomLaunchProfilesPanel::OnProfileEditDescription)
		.OnProfileModified(this, &SCustomLaunchProfilesPanel::OnProfileModified)
		.ChangeProfileEditorVisibility(this, &SCustomLaunchProfilesPanel::SetProfileEditorVisible)
		.EditPanelVisible(this, &SCustomLaunchProfilesPanel::IsProfileEditorVisible)
	]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchProfilesPanel::CreateProfileEditorWidget()
{
	PropertyEditor = SNew(SCustomLaunchCustomProfileEditor, Model.ToSharedRef());

	return SNew(SVerticalBox)
	.Visibility_Lambda( [this] { return Model->GetSelectedProfile().IsValid() && IsProfileEditorVisible() ? EVisibility::Visible :EVisibility::Collapsed; } )

	// extensions button bar
	+SVerticalBox::Slot()
	.AutoHeight()
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
		.Visibility(this, &SCustomLaunchProfilesPanel::GetExtensionsBarVisibility)
		[
			SNew(SHorizontalBox)

			// bannel label
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			.Padding(6,2)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TasksLabel", "Launch Tasks"))
			]

			// new task button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(4,2)
			[
				SNew(SComboButton)
				.ComboButtonStyle(FAppStyle::Get(), "ComboButton")
				.ButtonStyle(FAppStyle::Get(), "Button")
				.ToolTipText(LOCTEXT("CommandsToolTip", "Configure the commands to run."))
				.OnGetMenuContent(PropertyEditor.ToSharedRef(), &SCustomLaunchCustomProfileEditor::MakeCommandsMenu)
				.MenuPlacement(MenuPlacement_ComboBoxRight)
				.HasDownArrow(false)
				.ButtonContent()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1,3)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Plus"))
						.ColorAndOpacity(FStyleColors::AccentGreen)
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1,3)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CommandsLabel", "Add Task"))
					]
				]
			]
		]
	]


	// advanced profile warning banner
	+SVerticalBox::Slot()
	.AutoHeight()
	[
		CreateAdvancedProfileWarningWidget()
	]

	// profile property editor
	+SVerticalBox::Slot()
	.FillHeight(1)
	[
		SNew(SScrollBorder, PropertyEditor.ToSharedRef())
		[
			PropertyEditor.ToSharedRef()
		]
	]
	;

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchProfilesPanel::CreateProfileEditorToolbarWidget()
{
	return SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush("Brushes.Header"))
	.Padding(16)
	[
		SNew(SVerticalBox)
				
		// profile details & control buttons
		+SVerticalBox::Slot()
		.Padding(0)
		[
			SNew(SHorizontalBox)

			// profile icon
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4,0)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(44,44))
				.Image( this, &SCustomLaunchProfilesPanel::GetSelectedProfileImage )
			]

			// profile details
			+SHorizontalBox::Slot()
			.FillWidth(1)
			.Padding(4)
			[
				SNew(SVerticalBox)

				// profile name
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SNew(STextBlock)
					.Text( this, &SCustomLaunchProfilesPanel::GetSelectedProfileName )
					.OverflowPolicy(ETextOverflowPolicy::Clip)
				]

				// profile description
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(2)
				[
					SNew(STextBlock)
					.Text( this, &SCustomLaunchProfilesPanel::GetSelectedProfileDescription )
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				]
			]

			// launch button
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4,0,0,0)
			.VAlign(VAlign_Top)
			[
				SNew(SButton)
				.OnClicked( this, &SCustomLaunchProfilesPanel::OnLaunchButtonClicked )
				.IsEnabled( this, &SCustomLaunchProfilesPanel::IsLaunchButtonEnabled )
				.ButtonStyle(FAppStyle::Get(), "PrimaryButton")
				.ToolTipText( this, &SCustomLaunchProfilesPanel::GetLaunchButtonToolTipText )
				.ContentPadding(4)
				[
					SNew(SHorizontalBox)
				
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1,0)
					[
						SNew(SImage)
						.Image( this, &SCustomLaunchProfilesPanel::GetLaunchButtonImage )
						.ColorAndOpacity(FSlateColor::UseForeground())
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(1,0)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("DeviceProxyLaunchButton", "Launch"))
					]
				]
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchProfilesPanel::CreateAdvancedProfileWarningWidget()
{
	return SNew(SBorder)
	.BorderImage(FAppStyle::GetBrush("ToolPanel.LightGroupBorder"))
	.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Warning"))
	.Visibility_Lambda( [this]() { return Model->GetSelectedProfile().IsValid() && Model->IsAdvancedProfile(Model->GetSelectedProfile().ToSharedRef()) ? EVisibility::Visible : EVisibility::Collapsed; })
	.Padding(4)
	[
		SNew(SHorizontalBox)

		// Notice
		+SHorizontalBox::Slot()
		.FillWidth(1)
		.Padding(16, 0)
		.VAlign(VAlign_Center)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.Text(LOCTEXT("AdvancedProfileWarning", "Profile contains advanced/legacy settings"))
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.Text(LOCTEXT("AdvancedProfileWarningDetail", "These settings may be multiple cook platforms or unsupported packaging, deployment or launch modes. Please use Legacy Project Launcher to edit the advanced properties in this profile."))
				.AutoWrapText(true)
			]
		]

		// Button
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.OnClicked_Lambda( []() { FGlobalTabmanager::Get()->TryInvokeTab(FTabId("LegacyProjectLauncher")); return FReply::Handled(); })
			.ButtonStyle(FProjectLauncherStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("OpenDeviceProjectLauncherTip", "Open the legacy Project Launcher."))
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Launcher.TabIcon"))
			]
		]
	]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


EVisibility SCustomLaunchProfilesPanel::GetExtensionsBarVisibility() const
{
	if (IsProfileEditorVisible() && Model->AreExtensionsEnabled())
	{
		return EVisibility::Visible;
	}
	else
	{
		return EVisibility::Collapsed; 
	}
}


void SCustomLaunchProfilesPanel::OnProfileEdit(const ILauncherProfilePtr& Profile)
{
	Model->SelectProfile(Profile);
	SetProfileEditorVisible(true);
}




void SCustomLaunchProfilesPanel::OnProfileDelete(const ILauncherProfilePtr& Profile)
{
	if (Profile.IsValid() && !Model->IsBasicLaunchProfile(Profile))
	{
		FText MessageText = LOCTEXT("ProfileDeleteConfirm", "Are you sure you want to delete this profile?");
		if (ProjectLauncher::GetUserConfirmation(MessageText, FText::FromString(*Profile->GetName()), false))
		{
			Model->GetProfileManager()->RemoveProfile(Profile.ToSharedRef());
			Model->SelectProfile(Model->GetBasicLaunchProfile());
		}
	}
}


void SCustomLaunchProfilesPanel::OnProfileDuplicate(const ILauncherProfilePtr& Profile)
{
	if (Profile.IsValid() && !Model->IsBasicLaunchProfile(Profile))
	{
		ILauncherProfilePtr NewProfile = Model->CloneCustomProfile(Profile.ToSharedRef());

		if (NewProfile.IsValid())
		{
			Model->GetProfileManager()->AddProfile(NewProfile.ToSharedRef());

			Model->SelectProfile(NewProfile);
			SetProfileEditorVisible(true);

			ProfileSelector->StartEditProfileName(NewProfile);
		}
	}
}

void SCustomLaunchProfilesPanel::OnProfileRename(const ILauncherProfilePtr& Profile)
{
	if (Profile.IsValid() && !Model->IsBasicLaunchProfile(Profile))
	{
		Model->SelectProfile(Profile);

		ProfileSelector->StartEditProfileName(Profile);
	}
}

void SCustomLaunchProfilesPanel::OnProfileEditDescription(const ILauncherProfilePtr& Profile)
{
	if (Profile.IsValid() && !Model->IsBasicLaunchProfile(Profile))
	{
		Model->SelectProfile(Profile);

		ProfileSelector->StartEditProfileDescription(Profile);
	}
}


void SCustomLaunchProfilesPanel::OnProfileSave(const ILauncherProfilePtr& Profile)
{
	if (Profile.IsValid() && !Model->IsBasicLaunchProfile(Profile))
	{
		Model->GetProfileManager()->AddProfile(Profile.ToSharedRef());
	}
	else
	{
		checkNoEntry();
	}
}



FReply SCustomLaunchProfilesPanel::OnLaunchButtonClicked()
{
	OnProfileLaunchClicked.ExecuteIfBound(Model->GetSelectedProfile());
	return FReply::Handled();
}

bool SCustomLaunchProfilesPanel::IsLaunchButtonEnabled() const
{
	ILauncherProfilePtr Profile = Model->GetSelectedProfile();
	return Profile.IsValid() && Profile->IsValidForLaunch();
}

FText SCustomLaunchProfilesPanel::GetLaunchButtonToolTipText() const
{
	ILauncherProfilePtr Profile = Model->GetSelectedProfile();
	return ProjectLauncher::GetProfileLaunchErrorMessage(Profile, true);
}

const FSlateBrush* SCustomLaunchProfilesPanel::GetLaunchButtonImage() const
{
	ILauncherProfilePtr Profile = Model->GetSelectedProfile();

	if (!IsLaunchButtonEnabled())
	{
		return FAppStyle::Get().GetBrush("Icons.Error");
	}
	else if (Profile->GetAutomatedTests().Num() > 0)
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.TestAutomation");
	}
	else if (Profile->GetBuildCookRunCommands().Num() != 1)
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.Launch");
	}	
	// @fixme: figure out what the icon is like for multiple BCR steps
	else if (Profile->GetFirstBuildCookRun()->GetLaunchMode() != ELauncherProfileLaunchModes::DoNotLaunch)
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.Launch");
	}
	else if (Profile->GetFirstBuildCookRun()->GetDeploymentMode() != ELauncherProfileDeploymentModes::DoNotDeploy)
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.Deploy");
	}
	else if (Profile->GetFirstBuildCookRun()->GetPackagingMode() != ELauncherProfilePackagingModes::DoNotPackage)
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.Package");
	}
	else if (Profile->GetFirstBuildCookRun()->GetCookMode() != ELauncherProfileCookModes::DoNotCook)
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.Cook");
	}
	else if (Profile->GetFirstBuildCookRun()->GetBuildMode() != ELauncherProfileBuildModes::DoNotBuild)
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.Build");
	}
	else if (Profile->GetFirstBuildCookRun()->IsImportingZenSnapshot())
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.Zen");
	}
	else
	{
		return FProjectLauncherStyle::Get().GetBrush("Icons.Task.Launch");
	}
}



FReply SCustomLaunchProfilesPanel::OnCreateNewCustomProfileClicked()
{
	// create a new profile
	ILauncherProfileRef NewProfile = Model->CreateCustomProfile(TEXT("New Profile"));
	NewProfile->AssignId();
	Model->GetProfileManager()->AddProfile(NewProfile);

	Model->SelectProfile(NewProfile);

	// wait one tick then begin editing. this gives time for the list view to create the widgets for the new item
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, NewProfile]()
	{
		ProfileSelector->StartEditProfileName(NewProfile.ToSharedPtr());
	});

	return FReply::Handled();
}

FReply SCustomLaunchProfilesPanel::OnOpenDeviceManagerClicked()
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId("DeviceManager"));
	return FReply::Handled();
}

void SCustomLaunchProfilesPanel::OnProfileSelected( const ILauncherProfilePtr& NewProfile, const ILauncherProfilePtr& OldProfile)
{
	if (NewProfile.IsValid() && !Model->IsAdvancedProfile(NewProfile.ToSharedRef()))
	{
		PropertyEditor->SetProfile(NewProfile);
	}
	else
	{
		PropertyEditor->SetProfile(nullptr);
	}
}

void SCustomLaunchProfilesPanel::OnProfileModified(const ILauncherProfilePtr& Profile)
{
	if (!Model->IsBasicLaunchProfile(Profile.ToSharedRef()))
	{
		Model->GetProfileManager()->SaveJSONProfile(Profile.ToSharedRef());
	}
}



void SCustomLaunchProfilesPanel::OnProfileLaunchComplete()
{
	OutputLog->RefreshLog();
}


void SCustomLaunchProfilesPanel::OnLogAreaExpansionChanged( bool bExpanded )
{
	LogAreaSplitter->SlotAt(1).SetSizingRule( bExpanded ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent );

	LogAreaSplitter->SlotAt(0).SetResizable(bExpanded);
	LogAreaSplitter->SlotAt(1).SetResizable(bExpanded);
}

const FSlateBrush* SCustomLaunchProfilesPanel::GetSelectedProfileImage() const
{
	if (Model->GetSelectedProfile().IsValid() && Model->IsAdvancedProfile(Model->GetSelectedProfile().ToSharedRef()))
	{
		return FAppStyle::GetBrush("Icons.Warning.Large");
	}
	else
	{
		TArray<const PlatformInfo::FTargetPlatformInfo*> PlatformInfos = Model->GetPlatformInfosForAllCommands(Model->GetSelectedProfile());
		return FProjectLauncherStyle::GetProfileBrushForPlatforms(PlatformInfos, EPlatformIconSize::Large);
	}
}

bool SCustomLaunchProfilesPanel::IsSelectedProfileReadOnly() const
{
	return !Model->GetSelectedProfile().IsValid() || Model->IsBasicLaunchProfile(Model->GetSelectedProfile()); 
}

FText SCustomLaunchProfilesPanel::GetSelectedProfileName() const
{
	if (Model->IsBasicLaunchProfile(Model->GetSelectedProfile()))
	{
		const TSharedPtr<ITargetDeviceProxy> DeviceProxy = Model->GetDeviceProxy(Model->GetSelectedProfile()->GetFirstBuildCookRun().ToSharedRef());
		if (DeviceProxy.IsValid())
		{
			return FText::FromString(DeviceProxy->GetName());
		}
		else
		{
			return LOCTEXT("NoDevice", "No Device");
		}
	}
	else if (Model->GetSelectedProfile().IsValid())
	{
		return FText::FromString(Model->GetSelectedProfile()->GetName());
	}
	
	return FText::GetEmpty();
}



FText SCustomLaunchProfilesPanel::GetSelectedProfileDescription() const
{
	if (Model->IsBasicLaunchProfile(Model->GetSelectedProfile()))
	{
		const TSharedPtr<ITargetDeviceProxy> DeviceProxy = Model->GetDeviceProxy(Model->GetSelectedProfile()->GetFirstBuildCookRun().ToSharedRef());
		if (DeviceProxy.IsValid())
		{
			return DeviceProxy->GetPlatformDisplayName(NAME_None);
		}
		else
		{
			return FText::GetEmpty();
		}
	}
	else if (Model->GetSelectedProfile().IsValid())
	{
		return FText::FromString(Model->GetSelectedProfile()->GetDescription());
	}

	return FText::GetEmpty();
}





void SCustomLaunchProfilesPanel::SetProfileEditorVisibleCheckState(const ECheckBoxState NewCheckState)
{
	SetProfileEditorVisible( NewCheckState == ECheckBoxState::Checked );
}

ECheckBoxState SCustomLaunchProfilesPanel::GetProfileEditorVisibleCheckState() const
{
	return IsProfileEditorVisible() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SCustomLaunchProfilesPanel::SetProfileEditorVisible( bool bVisible )
{
	if (ProfileEditorSplitter.IsValid())
	{
		ProfileEditorSplitter->SlotAt(1).SetSizingRule( bVisible ? SSplitter::ESizeRule::FractionOfParent : SSplitter::ESizeRule::SizeToContent );
	
		ProfileEditorSplitter->SlotAt(0).SetResizable(bVisible);
		ProfileEditorSplitter->SlotAt(1).SetResizable(bVisible);

		if (bIsEditorOpen != bVisible)
		{
			bIsEditorOpen = bVisible;
			GConfig->SetBool(Model->GetConfigSection(), WasEditorOpenKey, bIsEditorOpen, Model->GetConfigIni());
		}
	}
}

bool SCustomLaunchProfilesPanel::IsProfileEditorVisible() const
{
	return !ProfileEditorSplitter.IsValid() || ProfileEditorSplitter->SlotAt(1).CanBeResized();

}


void SCustomLaunchProfilesPanel::SetDefaultProjectPath(FString ProjectPath)
{
	Model->GetProfileManager()->SetProjectPath(ProjectPath);
	Model->GetProfileManager()->SetBuildTarget(FString());
}

FString SCustomLaunchProfilesPanel::GetDefaultProjectPath() const
{
	return Model->GetProfileManager()->GetProjectPath();
}

void SCustomLaunchProfilesPanel::SetDefaultBuildTarget(TArray<FString> BuildTargets)
{
	FString BuildTarget = (BuildTargets.Num() > 0) ? BuildTargets[0] : FString(); 

	Model->GetProfileManager()->SetBuildTarget(BuildTarget);

	if (Model->GetSelectedProfile().IsValid())
	{
		for (const ILauncherProfileBuildCookRunRef& BuildCookRun : Model->GetSelectedProfile()->GetBuildCookRunCommands())
		{
			Model->UpdateCookedPlatformsFromBuildTarget(Model->GetSelectedProfile().ToSharedRef(), BuildCookRun);
		}
	}
}

TArray<FString> SCustomLaunchProfilesPanel::GetDefaultBuildTarget() const
{
	return TArray<FString>{Model->GetProfileManager()->GetBuildTarget()};
}



#undef LOCTEXT_NAMESPACE
