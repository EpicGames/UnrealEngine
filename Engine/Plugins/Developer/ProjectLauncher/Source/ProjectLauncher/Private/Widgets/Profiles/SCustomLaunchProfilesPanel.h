// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Model/ProjectLauncherModel.h"
#include "ILauncherWorker.h"
#include "UObject/ObjectPtr.h"

class SSplitter;
class SExpandableArea;
class SCustomLaunchCustomProfileSelector;

class SCustomLaunchProfilesPanel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomLaunchProfilesPanel) {}
		SLATE_EVENT(ProjectLauncher::FOnProfileClicked, OnProfileLaunchClicked)
	SLATE_END_ARGS()

public:
	SCustomLaunchProfilesPanel();
	~SCustomLaunchProfilesPanel();

public:
	void Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel);

	void OnProfileLaunchComplete();

private:
	TSharedRef<SWidget> CreateProfilesPanel();

	TSharedRef<SWidget> CreateProfileSelectorWidget();
	TSharedRef<SWidget> CreateProfileEditorWidget();
	TSharedRef<SWidget> CreateAdvancedProfileWarningWidget();
	TSharedRef<SWidget> CreateProfileEditorToolbarWidget();

	EVisibility GetExtensionsBarVisibility() const;

	void OnProfileEdit(const ILauncherProfilePtr& Profile);
	void OnProfileDelete(const ILauncherProfilePtr& Profile);
	void OnProfileSave(const ILauncherProfilePtr& Profile);
	void OnProfileDuplicate(const ILauncherProfilePtr& Profile);
	void OnProfileRename(const ILauncherProfilePtr& Profile);
	void OnProfileEditDescription(const ILauncherProfilePtr& Profile);
	void OnProfileSelected(const ILauncherProfilePtr& NewProfile, const ILauncherProfilePtr& OldProfile);
	void OnProfileModified(const ILauncherProfilePtr& Profile);

	void OnLogAreaExpansionChanged( bool bExpanded );

	FReply OnLaunchButtonClicked();
	bool IsLaunchButtonEnabled() const;
	FText GetLaunchButtonToolTipText() const;
	const FSlateBrush* GetLaunchButtonImage() const;

	FReply OnCreateNewCustomProfileClicked();

	FReply OnOpenDeviceManagerClicked();

	const FSlateBrush* GetSelectedProfileImage() const;
	bool IsSelectedProfileReadOnly() const;

	FText GetSelectedProfileName() const;
	FText GetSelectedProfileDescription() const;

	void SetProfileEditorVisibleCheckState(const ECheckBoxState NewCheckState);
	ECheckBoxState GetProfileEditorVisibleCheckState() const;

	void SetProfileEditorVisible(bool bVisible);
	bool IsProfileEditorVisible() const;

	void SetDefaultProjectPath(FString ProjectPath);
	FString GetDefaultProjectPath() const;

	void SetDefaultBuildTarget(TArray<FString> BuildTarget);
	TArray<FString> GetDefaultBuildTarget() const;

	ProjectLauncher::FOnProfileClicked OnProfileLaunchClicked;
	TSharedPtr<ProjectLauncher::FModel> Model;

	TSharedPtr<SSplitter> LogAreaSplitter;
	TSharedPtr<SExpandableArea> LogExpandableArea;
	TSharedPtr<SSplitter> ProfileEditorSplitter;
	TSharedPtr<SCustomLaunchCustomProfileSelector> ProfileSelector;
	TSharedPtr<class SCustomLaunchCustomProfileEditor> PropertyEditor;
	TSharedPtr<class SCustomLaunchOutputLog> OutputLog;


};
