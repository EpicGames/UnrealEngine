// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SProjectLauncher.h"
#include "SlateOptMacros.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Profiles/SCustomLaunchProfilesPanel.h"
#include "Widgets/Output/SCustomLaunchLaunchPanel.h"
#include "ILauncher.h"


#define LOCTEXT_NAMESPACE "SProjectLauncher"


SProjectLauncher::SProjectLauncher()
{
}


SProjectLauncher::~SProjectLauncher()
{
	if (LauncherWorker.IsValid())
	{
		LauncherWorker->Cancel();
		FPlatformProcess::Sleep(0.5f);
	}
}


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SProjectLauncher::Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel)
{
	Model = InModel;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Padding(0)
		[
			SAssignNew(LauncherPanelWidgetSwitcher, SWidgetSwitcher)
			.WidgetIndex(Panel_Profiles)
		
			// Profiles panel
			+SWidgetSwitcher::Slot()
			[
				SAssignNew(ProfileSelectorPanel, SCustomLaunchProfilesPanel, Model.ToSharedRef())
				.OnProfileLaunchClicked(this, &SProjectLauncher::OnProfileRun)
			]

			// Progress Panel
			+SWidgetSwitcher::Slot()
			[
				SAssignNew(LaunchProgressPanel, SCustomLaunchLaunchPanel, Model.ToSharedRef())
				.OnCloseClicked(this, &SProjectLauncher::OnProgressClose)
				.OnRerunClicked(this, &SProjectLauncher::OnRerunClicked)
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SProjectLauncher::OnProfileRun(const ILauncherProfilePtr& Profile)
{
	LaunchProgressPanel->ClearLog();

	LauncherProfile = Profile;
	if (LauncherProfile.IsValid())
	{
		Model->UpdateDeployDeviceProxyFromCookedPlatforms(LauncherProfile.ToSharedRef());


		LauncherWorker = Model->GetLauncher()->Launch(Model->GetDeviceProxyManager(), LauncherProfile.ToSharedRef());
	
		if (LauncherWorker.IsValid())
		{
			LaunchProgressPanel->SetLauncherWorker(LauncherWorker.ToSharedRef());
			LauncherPanelWidgetSwitcher->SetActiveWidgetIndex(Panel_Progress);
		}
	}
}



FReply SProjectLauncher::OnProgressClose()
{
	if (LauncherWorker.IsValid())
	{
		LauncherWorker->Cancel();
	}
	LauncherProfile.Reset();

	LauncherPanelWidgetSwitcher->SetActiveWidgetIndex(Panel_Profiles);
	ProfileSelectorPanel->OnProfileLaunchComplete();

	return FReply::Handled();
}



FReply SProjectLauncher::OnRerunClicked()
{
	LaunchProgressPanel->ClearLog();

	if (LauncherWorker.IsValid())
	{
		LauncherWorker->Cancel();
	}
	LauncherWorker = Model->GetLauncher()->Launch(Model->GetDeviceProxyManager(), LauncherProfile.ToSharedRef());

	if (LauncherWorker.IsValid())
	{
		LaunchProgressPanel->SetLauncherWorker(LauncherWorker.ToSharedRef());
	}

	return FReply::Handled();
}



#undef LOCTEXT_NAMESPACE
