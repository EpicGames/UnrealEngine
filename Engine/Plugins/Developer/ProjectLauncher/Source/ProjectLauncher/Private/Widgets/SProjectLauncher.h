// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Model/ProjectLauncherModel.h"
#include "ILauncherWorker.h"

class SWidgetSwitcher;
class SCustomLaunchLaunchPanel;
class SCustomLaunchProfilesPanel;

class SProjectLauncher : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SProjectLauncher) { }
	SLATE_END_ARGS()

public:
	SProjectLauncher();
	~SProjectLauncher();

public:
	void Construct(const FArguments& InArgs, const TSharedRef<ProjectLauncher::FModel>& InModel);

private:
	void OnProfileRun(const ILauncherProfilePtr& Profile);
	FReply OnProgressClose();
	FReply OnRerunClicked();

	ILauncherWorkerPtr LauncherWorker;
	ILauncherProfilePtr LauncherProfile;
	TSharedPtr<ProjectLauncher::FModel> Model;

	enum EPanel
	{
		Panel_Profiles = 0,
		Panel_Progress = 1,
	};

	TSharedPtr<SWidgetSwitcher> LauncherPanelWidgetSwitcher;
	TSharedPtr<SCustomLaunchProfilesPanel> ProfileSelectorPanel;
	TSharedPtr<SCustomLaunchLaunchPanel> LaunchProgressPanel;
};
