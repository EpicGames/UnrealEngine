// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchBuildTargetCombo.h"

#include "SlateOptMacros.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "DesktopPlatformModule.h"
#include "Misc/Paths.h"
#include "Settings/ProjectPackagingSettings.h"

#define LOCTEXT_NAMESPACE "SCustomLaunchBuildTargetCombo"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchBuildTargetCombo::Construct(const FArguments& InArgs, TSharedRef<ProjectLauncher::FModel> InModel)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	SelectedProject = InArgs._SelectedProject;
	SelectedBuildTargets = InArgs._SelectedBuildTargets;
	SetAdvancedPlatformsOption = InArgs._SetAdvancedPlatformsOption;
	GetAdvancedPlatformsOption = InArgs._GetAdvancedPlatformsOption;
	Model = InModel;

	FSlateFontInfo Font = InArgs._Font.IsSet() ? InArgs._Font.Get() : InArgs._TextStyle->Font;

	ChildSlot
	[
		SAssignNew(ComboButton, SComboButton)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SCustomLaunchBuildTargetCombo::GetBuildTargetName)
			.Font(Font)
		]
		.OnGetMenuContent(this, &SCustomLaunchBuildTargetCombo::MakeBuildTargetSelectionWidget)
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchBuildTargetCombo::MakeBuildTargetSelectionWidget()
{
	bool bAutoClose = !GetAdvancedPlatformsOption.IsSet() || !GetAdvancedPlatformsOption.Get(); // only keep the window open if we are in advanced mode (i.e. to allow multiselect)
	FMenuBuilder MenuBuilder(bAutoClose, nullptr);
	{
		EUserInterfaceActionType UIType = GetAdvancedPlatformsOption.Get() ? EUserInterfaceActionType::Check : EUserInterfaceActionType::RadioButton;

		FString DefaultBuildTarget = GetDefaultBuildTargetName();
		FText DefaultBuildTargetName = DefaultBuildTarget.IsEmpty() ? LOCTEXT("DefaultBuildTargetName", "Not Specified") : FText::Format(LOCTEXT("CurProjectDefaultBuildTargetName", "{0} (Project Default)"), FText::FromString(DefaultBuildTarget));
		MenuBuilder.AddMenuEntry(
			DefaultBuildTargetName, 
			LOCTEXT("DefaultBuildTargetActionHint", "Use the project default build target."), 
			FSlateIcon(), 
			FUIAction(
				FExecuteAction::CreateSP(this, &SCustomLaunchBuildTargetCombo::SetBuildTargetName, FString()),
				FCanExecuteAction(),
				FGetActionCheckState::CreateLambda( [this]() { return SelectedBuildTargets.Get().IsEmpty() || SelectedBuildTargets.Get().Contains(FString()) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			), 
			NAME_None, 
			UIType);


		if (SelectedProject.IsSet())
		{
			MenuBuilder.AddMenuSeparator();

			const TArray<FTargetInfo>& BuildTargets = FDesktopPlatformModule::Get()->GetTargetsForProject(SelectedProject.Get());
			for (const FTargetInfo& BuildTarget : BuildTargets)
			{
				if (BuildTarget.Type == EBuildTargetType::Program || BuildTarget.Type == EBuildTargetType::Editor)
				{
					continue;
				}

				MenuBuilder.AddMenuEntry(
					FText::FromString(BuildTarget.Name), 
					FText::FromString(BuildTarget.Path), 
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateSP(this, &SCustomLaunchBuildTargetCombo::SetBuildTargetName, BuildTarget.Name),
						FCanExecuteAction(),
						FGetActionCheckState::CreateSP(this, &SCustomLaunchBuildTargetCombo::GetBuildTargetCheckState, BuildTarget.Name)
					),
					NAME_None,
					UIType);
			}
		}

		if (SetAdvancedPlatformsOption.IsBound())
		{
			MenuBuilder.AddMenuSeparator();
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AdvancedPlatformLabel", "Advanced build target selection..."),
				LOCTEXT("AdvancedPlatformTip", "Select multiple platforms & build targets"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([this]()
					{
						ComboButton->SetIsOpen(false); // always close the popup menu when changing build target mode
						SetAdvancedPlatformsOption.ExecuteIfBound(!GetAdvancedPlatformsOption.Get()); 
					} ),
					FCanExecuteAction(),
					FGetActionCheckState::CreateLambda([this]() { return GetAdvancedPlatformsOption.Get() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; } )
				),
				NAME_None,
				EUserInterfaceActionType::Check);
		}
	}

	return MenuBuilder.MakeWidget();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


FText SCustomLaunchBuildTargetCombo::GetBuildTargetName() const
{
	if (SelectedProject.IsSet() && !Model->AreProjectSettingsReady(SelectedProject.Get()))
	{
		return FText::Format( LOCTEXT("TargetRefreshingLabel", "(loading {0} configuration...)"), FText::FromString(FPaths::GetBaseFilename(SelectedProject.Get())));
	}

	TArray<FString> BuildTargetNames = SelectedBuildTargets.Get();

	if (BuildTargetNames.Num() > 1)
	{
		return FText::Format( LOCTEXT("MultipleTargets", "Multiple Build Targets ({0})"), FText::FromString(FString::Join(BuildTargetNames, TEXT(", "))));
	}
	else
	{
		FString BuildTargetName = (BuildTargetNames.Num() > 0) ? BuildTargetNames[0] : FString();

		if (BuildTargetName.IsEmpty())
		{
			FString DefaultBuildTarget = GetDefaultBuildTargetName();
			if (DefaultBuildTarget.IsEmpty())
			{
				return LOCTEXT("DefaultBuildTargetName", "Not Specified");
			}
			else
			{
				return FText::Format(LOCTEXT("CurProjectDefaultBuildTargetName", "{0} (Project Default)"), FText::FromString(DefaultBuildTarget));
			}
		}

		return FText::FromString(BuildTargetName);
	}
}

void SCustomLaunchBuildTargetCombo::SetBuildTargetName(FString BuildTargetName)
{
	TArray<FString> BuildTargets;
	if (!BuildTargetName.IsEmpty())
	{
		if (GetAdvancedPlatformsOption.Get())
		{
			BuildTargets = SelectedBuildTargets.Get();
		}

		BuildTargets.Remove(FString());
		if (BuildTargets.Contains(BuildTargetName))
		{
			BuildTargets.Remove(BuildTargetName);
		}
		else
		{
			BuildTargets.Add(BuildTargetName);
		}
	}

	OnSelectionChanged.ExecuteIfBound(BuildTargets);
}

FString SCustomLaunchBuildTargetCombo::GetDefaultBuildTargetName() const
{
	if (SelectedProject.IsSet())
	{
		return Model->GetProjectSettings(SelectedProject.Get()).DefaultBuildTargetName;
	}

	return FString();
}

ECheckBoxState SCustomLaunchBuildTargetCombo::GetBuildTargetCheckState( FString BuildTargetName ) const
{
	bool bResult = false;
	const TArray<FString>& BuildTargets = SelectedBuildTargets.Get();

	if (BuildTargetName.IsEmpty())
	{
		bResult = (BuildTargets.Num() == 0);
	}
	else
	{
		bResult = BuildTargets.Contains(BuildTargetName);
	}

	return bResult ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}


#undef LOCTEXT_NAMESPACE
