// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchContentSchemeCombo.h"

#include "SlateOptMacros.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SCustomLaunchContentSchemeCombo"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchContentSchemeCombo::Construct(const FArguments& InArgs)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	IsContentSchemeAvailable = InArgs._IsContentSchemeAvailable;
	SelectedContentScheme = InArgs._SelectedContentScheme;

	FSlateFontInfo Font = InArgs._Font.IsSet() ? InArgs._Font.Get() : InArgs._TextStyle->Font;

	ChildSlot
	[
		SNew(SComboButton)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SCustomLaunchContentSchemeCombo::GetContentSchemeName)
			.Font(Font)
		]
		.OnGetMenuContent(this, &SCustomLaunchContentSchemeCombo::MakeContentSchemeSelectionWidget)
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchContentSchemeCombo::MakeContentSchemeSelectionWidget()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	{
		for (ProjectLauncher::EContentScheme ContentScheme : ProjectLauncher::GetAllContentSchemes())
		{
			FText Reason = FText::GetEmpty();
			bool bIsAvailable = !IsContentSchemeAvailable.IsBound() || IsContentSchemeAvailable.Execute(ContentScheme, Reason);

			if (bIsAvailable)
			{
				MenuBuilder.AddMenuEntry(
					ProjectLauncher::GetContentSchemeDisplayName(ContentScheme),
					ProjectLauncher::GetContentSchemeToolTip(ContentScheme),
					FSlateIcon(), 
					FUIAction(
						FExecuteAction::CreateSP(this, &SCustomLaunchContentSchemeCombo::SetContentScheme, ContentScheme),
						FCanExecuteAction(),
						FGetActionCheckState::CreateLambda( [this, ContentScheme]() { return (SelectedContentScheme.Get() == ContentScheme) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					),
					NAME_None,
					EUserInterfaceActionType::Check);
			}
			else if (Reason.IsEmpty())
			{
				// no reason given - hide the item completely
				continue;
			}
			else
			{
				MenuBuilder.AddMenuEntry(
					ProjectLauncher::GetContentSchemeDisplayName(ContentScheme),
					Reason,
					FSlateIcon(), 
					FUIAction(
						FExecuteAction(),
						FCanExecuteAction::CreateLambda( []() { return false; } ),
						FGetActionCheckState::CreateLambda( [this, ContentScheme]() { return (SelectedContentScheme.Get() == ContentScheme) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					),
					NAME_None,
					EUserInterfaceActionType::Check);
			}
		}
	}

	return MenuBuilder.MakeWidget();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


FText SCustomLaunchContentSchemeCombo::GetContentSchemeName() const
{
	ProjectLauncher::EContentScheme ContentScheme = SelectedContentScheme.Get();
	return ProjectLauncher::GetContentSchemeDisplayName(ContentScheme);
}

void SCustomLaunchContentSchemeCombo::SetContentScheme(ProjectLauncher::EContentScheme ContentScheme)
{
	OnSelectionChanged.ExecuteIfBound(ContentScheme);
}


#undef LOCTEXT_NAMESPACE
