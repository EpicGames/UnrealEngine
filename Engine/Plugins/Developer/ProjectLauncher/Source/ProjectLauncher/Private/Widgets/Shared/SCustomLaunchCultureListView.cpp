// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchCultureListView.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Views/STableRow.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/ProjectLauncherStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Culture.h"
#include "Internationalization/Internationalization.h"


#define LOCTEXT_NAMESPACE "SCustomLaunchCultureListView"


namespace
{
	static const TArray<FString> Group_Empty;
	static const TArray<FString> Group_EFIGS   { TEXT("en"), TEXT("fr"), TEXT("it"), TEXT("de"), TEXT("es") };
	static const TArray<FString> Group_CJK     { TEXT("zh"), TEXT("ja"), TEXT("ko") };
	static const TArray<FString> Group_Nordics { TEXT("sv"), TEXT("nb"), TEXT("da"), TEXT("fi"), TEXT("is") };
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchCultureListView::Construct(const FArguments& InArgs, TSharedRef<ProjectLauncher::FModel> InModel)
{
	Model = InModel.ToSharedPtr();
	OnSelectionChanged = InArgs._OnSelectionChanged;
	SelectedCultures = InArgs._SelectedCultures;
	ProjectPath = InArgs._ProjectPath;

	// start out only showing the current selection if there is one, for easier at-a-glance verification
	LocalizationGroupFilter = SelectedCultures.Get().Num() == 0 ? ELocalizationGroupFilter::All : ELocalizationGroupFilter::Selected;

    LocalizationGroupDetails.Emplace(ELocalizationGroupFilter::All,			FLocalizationGroup(LOCTEXT("LocGroupName_All",     "All"),					Group_Empty		));
    LocalizationGroupDetails.Emplace(ELocalizationGroupFilter::Selected,	FLocalizationGroup(LOCTEXT("LocGroupName_Selected","Selected Cultures"),	Group_Empty		));
    LocalizationGroupDetails.Emplace(ELocalizationGroupFilter::EFIGS,		FLocalizationGroup(LOCTEXT("LocGroupName_EFIGS",   "EFIGS"),				Group_EFIGS		));
    LocalizationGroupDetails.Emplace(ELocalizationGroupFilter::CJK,			FLocalizationGroup(LOCTEXT("LocGroupName_CJK",     "CJK"),					Group_CJK		));
    LocalizationGroupDetails.Emplace(ELocalizationGroupFilter::Nordics,		FLocalizationGroup(LOCTEXT("LocGroupName_Nordics", "Nordics"),				Group_Nordics	));
	check(LocalizationGroupDetails.Num() == (int32)ELocalizationGroupFilter::MAX);

	ChildSlot
	[
		SNew(SBorder)
		.Padding(2)
		.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
		[
			SAssignNew(CultureListView, STreeView<FCultureTreeNodePtr>)
			.TreeItemsSource(&CultureListViewItemsSource)
			.OnGenerateRow(this, &SCustomLaunchCultureListView::GenerateCultureTreeNodeRow)
			.OnGetChildren(this, &SCustomLaunchCultureListView::GetCultureTreeNodeChildren)
		]
	];

	RefreshCultureList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION






BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchCultureListView::MakeControlsWidget()
{
	FMenuBuilder LocalizationGroupMenuBuilder(true, nullptr);
	for (int32 GroupIndex = 0; GroupIndex < (int32)ELocalizationGroupFilter::MAX; GroupIndex++)
	{
		ELocalizationGroupFilter Group = (ELocalizationGroupFilter)GroupIndex;

		FUIAction Action(
			FExecuteAction::CreateLambda( [this,Group]()
			{
				LocalizationGroupFilter = Group;
				RefreshCultureList();
			}),
			FCanExecuteAction(),
			FGetActionCheckState::CreateLambda( [this,Group]()
			{
				return (LocalizationGroupFilter == Group) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
		);

		LocalizationGroupMenuBuilder.AddMenuEntry(LocalizationGroupDetails[Group].DisplayName, LOCTEXT("LocGroup_Tip", "Show only cultures in this group"), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::RadioButton);
	}


	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SComboButton)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text_Lambda( [this]() { return LocalizationGroupDetails[LocalizationGroupFilter].DisplayName; } )
			]
			.MenuContent()
			[
				LocalizationGroupMenuBuilder.MakeWidget()
			]
		]


		+SHorizontalBox::Slot()
		.FillWidth(1)
		.Padding(4,0)
		[
			SNew(SSearchBox)
			.OnTextCommitted(this, &SCustomLaunchCultureListView::OnSearchFilterTextCommitted)
			.OnTextChanged(this, &SCustomLaunchCultureListView::OnSearchFilterTextChanged)
		]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SCustomLaunchCultureListView::GetCultureTreeNodeChildren(FCultureTreeNodePtr Node, TArray<FCultureTreeNodePtr>& OutChildren)
{
	OutChildren = Node->Children;
}



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<ITableRow> SCustomLaunchCultureListView::GenerateCultureTreeNodeRow(FCultureTreeNodePtr Node, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FCultureTreeNodePtr>, OwnerTable)
	.ToolTipText(this, &SCustomLaunchCultureListView::GetCultureDisplayName, Node)
	[
		SNew(SCheckBox)
		.Padding(2)
		.IsChecked(this, &SCustomLaunchCultureListView::GetCultureTreeNodeCheckState, Node)
		.OnCheckStateChanged(this, &SCustomLaunchCultureListView::SetCultureTreeNodeCheckState, Node)
		[
			SNew(STextBlock)
			.Text(FText::FromString(*Node->Name))
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



FText SCustomLaunchCultureListView::GetCultureDisplayName(FCultureTreeNodePtr Item) const
{
	const FCulturePtr CulturePtr = FInternationalization::Get().GetCulture(Item->Name);
	if (CulturePtr == nullptr)
	{
		return FText::GetEmpty();
	}

	return FText::FromString(CulturePtr->GetDisplayName());
}

ECheckBoxState SCustomLaunchCultureListView::GetCultureTreeNodeCheckState(FCultureTreeNodePtr Node) const
{
	return SelectedCultures.Get().Contains(Node->Name) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}



void SCustomLaunchCultureListView::SetCultureTreeNodeCheckState(ECheckBoxState CheckBoxState, FCultureTreeNodePtr Node)
{
	TArray<FString> CheckedCultures = SelectedCultures.Get();

	if (CheckBoxState == ECheckBoxState::Checked)
	{
		CheckedCultures.Add(Node->Name);
	}
	else
	{
		CheckedCultures.Remove(Node->Name);
	}

	OnSelectionChanged.ExecuteIfBound(CheckedCultures);
}



void SCustomLaunchCultureListView::OnSearchFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitType)
{
	CurrentFilterText = SearchText.ToString();
	RefreshCultureList();
}



void SCustomLaunchCultureListView::OnSearchFilterTextChanged(const FText& SearchText)
{
	CurrentFilterText = SearchText.ToString();
	if (CurrentFilterText.IsEmpty())
	{
		RefreshCultureList();
	}
}



void SCustomLaunchCultureListView::OnProjectChanged()
{
	RefreshCultureList();
}



void SCustomLaunchCultureListView::RefreshCultureList()
{
	bCultureListDirty = true;
}

void SCustomLaunchCultureListView::RefreshCultureListInternal()
{
	bCultureListDirty = false;

	CultureTreeRoot = MakeShared<FCultureTreeNode>();

	TArray<FString> SelectedCultureNames = SelectedCultures.Get();

	TArray<FString> CultureNames;
	FInternationalization::Get().GetCultureNames(CultureNames);
	CultureNames.Sort();

	for (const FString& Culture : CultureNames)
	{
		FString Language, Region;
		if (!Culture.Split(TEXT("-"), &Language, &Region))
		{
			Language = Culture;
		}

		// ignore this culture if it isn't filtered
		if (!CurrentFilterText.IsEmpty())
		{
			const FCulturePtr CulturePtr = FInternationalization::Get().GetCulture(Culture);
			if (!Culture.Contains(CurrentFilterText) && (CulturePtr == nullptr || !CulturePtr->GetDisplayName().Contains(CurrentFilterText)))
			{
				continue;
			}
		}

		// check the localization group filter table
		if (LocalizationGroupFilter == ELocalizationGroupFilter::Selected)
		{
			if (!SelectedCultureNames.Contains(Culture))
			{
				continue;
			}
		}
		else if (LocalizationGroupFilter != ELocalizationGroupFilter::All)
		{
			if (!LocalizationGroupDetails[LocalizationGroupFilter].Languages.Contains(Language))
			{
				continue;
			}
		}


		// add the culture, splitting into Language/Region hierarchy as necessary
		if (Region.IsEmpty())
		{
			FCultureTreeNodePtr LanguageNode = CultureTreeRoot->Children.Add_GetRef(MakeShared<FCultureTreeNode>());
			LanguageNode->Name = Culture;
		}
		else
		{
			// add or find the language
			FCultureTreeNodePtr LanguageNode = nullptr;
			if (FCultureTreeNodePtr* ChildPtr = CultureTreeRoot->Children.FindByPredicate( [Language]( const FCultureTreeNodePtr Node ) { return Node->Name == Language; } ))
			{
				LanguageNode = *ChildPtr;
			}
			else
			{
				LanguageNode = CultureTreeRoot->Children.Add_GetRef(MakeShared<FCultureTreeNode>());
				LanguageNode->Name = Language;
			}

			// add the region/country as a child
			FCultureTreeNodePtr RegionNode = LanguageNode->Children.Add_GetRef(MakeShared<FCultureTreeNode>());
			RegionNode->Name = Culture;
		}
	}


	CultureListViewItemsSource = CultureTreeRoot->Children;
	if (CultureListView.IsValid())
	{
		CultureListView->RequestTreeRefresh();
	}

	// expand the parents of anything that is selected
	for (FCultureTreeNodePtr LanguageNode : CultureTreeRoot->Children)
	{
		for (FCultureTreeNodePtr RegionNode : LanguageNode->Children)
		{
			if (SelectedCultureNames.Contains(RegionNode->Name))
			{
				CultureListView->SetItemExpansion(LanguageNode, true);
				break;
			}
		}
	}
}



void SCustomLaunchCultureListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bCultureListDirty && bHasPaintedThisFrame)
	{
		RefreshCultureListInternal();
	}

	bHasPaintedThisFrame = false;
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

int32 SCustomLaunchCultureListView::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	bHasPaintedThisFrame = true;
    return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}




#undef LOCTEXT_NAMESPACE
