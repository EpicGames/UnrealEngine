// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchDeviceCombo.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/ProjectLauncherStyle.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "PlatformInfo.h"
#include "Model/ProjectLauncherModel.h"

#define LOCTEXT_NAMESPACE "SCustomLaunchDeviceCombo"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchDeviceCombo::Construct(const FArguments& InArgs)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	SelectedDevices = InArgs._SelectedDevices;
	Platforms = InArgs._Platforms;
	bAllPlatforms = InArgs._AllPlatforms;

	ChildSlot
	[
		SNew(SHorizontalBox)
		
		+SHorizontalBox::Slot()
		.AutoWidth()		
		[
			SAssignNew(DeviceComboBox, SComboBox<TSharedPtr<FString>>)
			.OptionsSource(&AllDeviceIDs)
			.OnGenerateWidget(this, &SCustomLaunchDeviceCombo::GenerateDeviceListWidget)
			.OnSelectionChanged( this, &SCustomLaunchDeviceCombo::OnDeviceSelectionChanged)
			[
				SNew(SHorizontalBox)
					
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4,0)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16,16))
					.Image(this, &SCustomLaunchDeviceCombo::GetSelectedDeviceBrush)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4,0)
				[
					SNew(STextBlock)
					//.TextStyle( FAppStyle::Get(), "SmallText")
					.Text(this, &SCustomLaunchDeviceCombo::GetSelectedDeviceName)
				]
			]
		]
	];

	SCustomLaunchDeviceWidgetBase::Construct();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION






BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchDeviceCombo::GenerateDeviceListWidget( TSharedPtr<FString> DeviceID ) const
{
	const PlatformInfo::FTargetPlatformInfo* PlatformInfo = GetPlatformInfoForDeviceID(*DeviceID);
	const TSharedPtr<ITargetDeviceProxy> DeviceProxy = GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(*DeviceID);

	return SNew(SHorizontalBox)

		// platform icon
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4,0)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(16,16))
			.Image(PlatformInfo ? FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)) : FStyleDefaults::GetNoBrush())
		]

		// device name
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4,0)
		[
			SNew(STextBlock)
			.Text(GetDisplayNameForDevice(DeviceID))
			.ColorAndOpacity( DeviceProxy.IsValid() ? GetForegroundColor() : GetDisabledForegroundColor() )
		]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


const FSlateBrush* SCustomLaunchDeviceCombo::GetSelectedDeviceBrush() const
{
	TArray<FString> DeviceIDs = SelectedDevices.Get();

	if (DeviceIDs.Num() == 1)
	{
		FString DeviceID = DeviceIDs[0];
		const TSharedPtr<ITargetDeviceProxy> DeviceProxy = GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(DeviceID);
		if (DeviceProxy.IsValid())
		{
			const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(*DeviceProxy->GetTargetPlatformName(NAME_None));
			if (PlatformInfo != nullptr)
			{
				return FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal));
			}
		}
	}
	else if (DeviceIDs.Num() > 1)
	{
		return FAppStyle::Get().GetBrush("Icons.WarningWithColor");
	}

	return FStyleDefaults::GetNoBrush();
}

FText SCustomLaunchDeviceCombo::GetSelectedDeviceName() const
{
	TArray<FString> DeviceIDs = SelectedDevices.Get();

	if (DeviceIDs.Num() == 1)
	{
		FString DeviceID = DeviceIDs[0];
		const TSharedPtr<ITargetDeviceProxy> DeviceProxy = GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(DeviceID);
		if (DeviceProxy.IsValid())
		{
			return FText::FromString(*DeviceProxy->GetName());
		}
	}
	else if (DeviceIDs.Num() > 1)
	{
		return LOCTEXT("TooManyDevices", "Multiple devices (unsupported)");
	}

	return LOCTEXT("NoDevice", "(no device)");
}

void SCustomLaunchDeviceCombo::OnDeviceSelectionChanged(TSharedPtr<FString> DeviceID, ESelectInfo::Type InSelectInfo)
{
	TArray<FString> DeviceIDs;

	DeviceIDs.Add(*DeviceID);

	OnSelectionChanged.ExecuteIfBound(DeviceIDs);
}

#undef LOCTEXT_NAMESPACE
