// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchDeviceListView.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Views/STableRow.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/ProjectLauncherStyle.h"
#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "PlatformInfo.h"
#include "Model/ProjectLauncherModel.h"

#define LOCTEXT_NAMESPACE "SCustomLaunchDeviceListView"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchDeviceListView::Construct(const FArguments& InArgs)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	SelectedDevices = InArgs._SelectedDevices;
	Platforms = InArgs._Platforms;
	bAllPlatforms = InArgs._AllPlatforms;
	bSingleSelect = InArgs._SingleSelect;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(2)
		.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
		[
			SAssignNew(DeviceListView, SListView<TSharedPtr<FString>>)
			.ListItemsSource(&AllDeviceIDs)
			.OnGenerateRow(this, &SCustomLaunchDeviceListView::GenerateDeviceRow)
		]
	];

	SCustomLaunchDeviceWidgetBase::Construct();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION




BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<ITableRow> SCustomLaunchDeviceListView::GenerateDeviceRow(TSharedPtr<FString> DeviceID, const TSharedRef<STableViewBase>& OwnerTable)
{
	const PlatformInfo::FTargetPlatformInfo* PlatformInfo = GetPlatformInfoForDeviceID(*DeviceID);
	const TSharedPtr<ITargetDeviceProxy> DeviceProxy = GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(*DeviceID);

	TSharedPtr<SWidget> DeviceSelectorCheckbox = SNew(SCheckBox)
	.IsChecked(this, &SCustomLaunchDeviceListView::IsDeviceChecked, DeviceID)
	.OnCheckStateChanged(this, &SCustomLaunchDeviceListView::OnDeviceCheckStateChanged, DeviceID)
	.Style(FAppStyle::Get(), bSingleSelect ? "RadioButton" : "Checkbox")
	[
		SNew(SHorizontalBox)

		// platform icon
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4,0)
		[
			SNew(SImage)
			.DesiredSizeOverride(FVector2D(16,16))
			.Image(PlatformInfo ? FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)) : FStyleDefaults::GetNoBrush())
		]

		// device name
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(4,0)
		[
			SNew(STextBlock)
			.Text(GetDisplayNameForDevice(DeviceID))
			.ColorAndOpacity( DeviceProxy.IsValid() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground() )
		]
	];

	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
	.Padding(FMargin(4,1))
	[
		DeviceSelectorCheckbox.ToSharedRef()
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


ECheckBoxState SCustomLaunchDeviceListView::IsDeviceChecked(TSharedPtr<FString> DeviceID) const
{
	if (SelectedDevices.Get().Contains(*DeviceID))
	{
		return ECheckBoxState::Checked;
	}
	else
	{
		return ECheckBoxState::Unchecked;
	}
}



void SCustomLaunchDeviceListView::OnDeviceCheckStateChanged(ECheckBoxState NewState, TSharedPtr<FString> DeviceID)
{
	TArray<FString> Devices;
	if (!bSingleSelect)
	{
		Devices = SelectedDevices.Get();
	}
	else if (NewState != ECheckBoxState::Checked)
	{
		// do not allow the current item to be deselected in single-select mode
		return;
	}

	if (NewState == ECheckBoxState::Checked)
	{
		Devices.Add(*DeviceID);
	}
	else
	{
		Devices.Remove(*DeviceID);
	}

	OnSelectionChanged.ExecuteIfBound(Devices);
}


void SCustomLaunchDeviceListView::OnDeviceListRefreshed()
{
	DeviceListView->RequestListRefresh();
}





#undef LOCTEXT_NAMESPACE
