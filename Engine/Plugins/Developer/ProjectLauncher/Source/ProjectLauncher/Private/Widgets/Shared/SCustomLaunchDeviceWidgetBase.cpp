// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchDeviceWidgetBase.h"

#include "ITargetDeviceProxy.h"
#include "ITargetDeviceProxyManager.h"
#include "ITargetDeviceServicesModule.h"
#include "PlatformInfo.h"
#include "Modules/ModuleManager.h"
#include "Algo/StableSort.h"
#include "Model/ProjectLauncherModel.h"

void SCustomLaunchDeviceWidgetBase::Construct()
{
	const TSharedRef<ITargetDeviceProxyManager> DeviceProxyManager = GetDeviceProxyManager();
	DeviceProxyManager->OnProxyAdded().AddSP(this, &SCustomLaunchDeviceWidgetBase::OnDeviceProxyAdded);
	DeviceProxyManager->OnProxyRemoved().AddSP(this, &SCustomLaunchDeviceWidgetBase::OnDeviceProxyRemoved);

	RefreshDeviceList();
}


SCustomLaunchDeviceWidgetBase::~SCustomLaunchDeviceWidgetBase()
{
	const TSharedRef<ITargetDeviceProxyManager>& DeviceProxyManager = GetDeviceProxyManager();
	DeviceProxyManager->OnProxyAdded().RemoveAll(this);
	DeviceProxyManager->OnProxyRemoved().RemoveAll(this);
}


void SCustomLaunchDeviceWidgetBase::OnDeviceProxyAdded(const TSharedRef<ITargetDeviceProxy>& DeviceProxy)
{
	RefreshDeviceList();
}



void SCustomLaunchDeviceWidgetBase::OnDeviceProxyRemoved(const TSharedRef<ITargetDeviceProxy>& DeviceProxy)
{
	RefreshDeviceList();
}



void SCustomLaunchDeviceWidgetBase::RefreshDeviceList()
{
	// gather device proxies for all required platforms
	TArray<FString> RequiredPlatforms = Platforms.Get();
	TArray<TSharedPtr<ITargetDeviceProxy>> DeviceProxyList;
	if (bAllPlatforms)
	{
		for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : PlatformInfo::GetVanillaPlatformInfoArray())
		{
			TArray<TSharedPtr<ITargetDeviceProxy>> PlatformDeviceProxyList;
			GetDeviceProxyManager()->GetProxies(PlatformInfo->Name, false, PlatformDeviceProxyList);

			for (TSharedPtr<ITargetDeviceProxy> DeviceProxy : PlatformDeviceProxyList)
			{
				DeviceProxyList.AddUnique(DeviceProxy);
			}
		}
	}
	else
	{
		for (const FString& Platform : RequiredPlatforms)
		{
			const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(Platform));
			if (PlatformInfo != nullptr)
			{
				TArray<TSharedPtr<ITargetDeviceProxy>> PlatformDeviceProxyList;
				GetDeviceProxyManager()->GetProxies(PlatformInfo->Name, false, PlatformDeviceProxyList);

				for (TSharedPtr<ITargetDeviceProxy> DeviceProxy : PlatformDeviceProxyList)
				{
					DeviceProxyList.AddUnique(DeviceProxy);
				}
			}
		}
	}

	// remember all selected devices we've seeen
	for (const FString& DeviceID : SelectedDevices.Get())
	{
		KnownDeviceIDs.Add(DeviceID);
	}

	// add the device proxies
	TArray<FString> DeviceIDs;
	for (const TSharedPtr<ITargetDeviceProxy>& DeviceProxy : DeviceProxyList)
	{
		FString DeviceID = DeviceProxy->GetTargetDeviceId(NAME_None);
		DeviceIDs.Add(DeviceID);
		KnownDeviceIDs.Add(DeviceID);
	}

	// add all known devices (these will include devices that have been disconnected or not discovered yet)
	for (const FString& DeviceID : KnownDeviceIDs)
	{
		if (!DeviceIDs.Contains(DeviceID))
		{
			const PlatformInfo::FTargetPlatformInfo* PlatformInfo = GetPlatformInfoForDeviceID(DeviceID);
			if (bAllPlatforms || PlatformInfo == nullptr || RequiredPlatforms.Contains(PlatformInfo->Name))
			{
				DeviceIDs.Add(DeviceID);
			}
		}
	}
	
	// finalize the sorted device id list (note: these will be sorted by platform@devicename, so each platforms' devices will be grouped together)
	Algo::StableSort(DeviceIDs);
	AllDeviceIDs.Reset();
	for (const FString& DeviceID : DeviceIDs)
	{
		AllDeviceIDs.Add(MakeShared<FString>(DeviceID));
	}

	OnDeviceListRefreshed();
}


void SCustomLaunchDeviceWidgetBase::OnSelectedPlatformChanged()
{
	// collect all device ids for the new platforms
	TArray<FString> ValidPlatformDevices;
	for (const FString& Platform : Platforms.Get())
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(Platform));
		if (PlatformInfo != nullptr)
		{
			TArray<TSharedPtr<ITargetDeviceProxy>> PlatformDeviceProxyList;
			GetDeviceProxyManager()->GetProxies(PlatformInfo->Name, false, PlatformDeviceProxyList);

			for (TSharedPtr<ITargetDeviceProxy> PlatformDeviceProxy : PlatformDeviceProxyList)
			{
				ValidPlatformDevices.Add(PlatformDeviceProxy->GetTargetDeviceId(NAME_None));
			}
		}
	}

	// remove any devices that belong to other platforms
	KnownDeviceIDs.Reset();
	TArray<FString> ValidSelectedDevices = SelectedDevices.Get();
	int32 InvalidCount = ValidSelectedDevices.RemoveAll( [ValidPlatformDevices]( const FString& Device )
	{
		return !ValidPlatformDevices.Contains(Device);
	});

	if (InvalidCount > 0)
	{
		OnSelectionChanged.ExecuteIfBound(ValidSelectedDevices);
	}

	// update the list
	RefreshDeviceList();
}


const TSharedRef<ITargetDeviceProxyManager> SCustomLaunchDeviceWidgetBase::GetDeviceProxyManager() const
{
	ITargetDeviceServicesModule& TargetDeviceServicesModule = FModuleManager::LoadModuleChecked<ITargetDeviceServicesModule>("TargetDeviceServices");
	return TargetDeviceServicesModule.GetDeviceProxyManager();
}


const PlatformInfo::FTargetPlatformInfo* SCustomLaunchDeviceWidgetBase::GetPlatformInfoForDeviceID( const FString& DeviceID ) const
{
	const TSharedPtr<ITargetDeviceProxy> DeviceProxy = GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(DeviceID);
	if (DeviceProxy.IsValid())
	{
		return PlatformInfo::FindPlatformInfo(*DeviceProxy->GetTargetPlatformName(NAME_None));
	}

	FString PlatformName;
	if (DeviceID.Split(TEXT("@"), &PlatformName, nullptr))
	{
		return PlatformInfo::FindPlatformInfo(FName(*PlatformName));
	}

	return nullptr;
}


FText SCustomLaunchDeviceWidgetBase::GetDisplayNameForDevice( const TSharedPtr<FString>& DeviceID ) const
{
	const TSharedPtr<ITargetDeviceProxy> DeviceProxy = GetDeviceProxyManager()->FindProxyDeviceForTargetDevice(*DeviceID);
	if (DeviceProxy.IsValid())
	{
		return FText::FromString(DeviceProxy->GetName());
	}

	FString DeviceName;
	if (!DeviceID->Split(TEXT("@"), nullptr, &DeviceName))
	{
		DeviceName = *DeviceID;
	}

	return FText::FromString(DeviceName);
}

