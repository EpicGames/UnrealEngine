// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchMapListView.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Views/STableRow.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "Styling/ProjectLauncherStyle.h"
#include "Misc/PackageName.h"
#include "Interfaces/IPluginManager.h"


#define LOCTEXT_NAMESPACE "SCustomLaunchMapListView"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchMapListView::Construct(const FArguments& InArgs, TSharedRef<ProjectLauncher::FModel> InModel)
{
	Model = InModel.ToSharedPtr();
	OnSelectionChanged = InArgs._OnSelectionChanged;
	SelectedMaps = InArgs._SelectedMaps;
	ProjectPath = InArgs._ProjectPath;
	OnMapFilterChanged = InArgs._OnMapFilterChanged;
	MapSourceFlags = InArgs._MapSourceFlags;

	ChildSlot
	[
		SNew(SBorder)
		.Padding(2)
		.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
		[
			SAssignNew(MapTreeView, STreeView<FMapTreeNodePtr>)
			.TreeItemsSource(&MapTreeViewItemsSource)
			.OnGenerateRow(this, &SCustomLaunchMapListView::GenerateMapTreeNodeRow)
			.OnGetChildren(this, &SCustomLaunchMapListView::GetMapTreeNodeChildren)
		]
	];

	RefreshMapList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION






BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchMapListView::MakeControlsWidget()
{
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style( FAppStyle::Get(), "ToggleButtonCheckBox" )
			.IsChecked_Lambda( [this]() { return bShowFolders ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda( [this](ECheckBoxState CheckBoxState)
			{
				bShowFolders = CheckBoxState == ECheckBoxState::Checked;
				RefreshMapList();
			})
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16,16))
				.Image( FAppStyle::GetBrush("Icons.FolderClosed")) // @todo: not the best image
			]
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(8, 0, 0, 0)
		[
			SNew(SComboButton)
			.HasDownArrow(true)
			.ToolTipText(LOCTEXT("FilterMenuTooltip", "Toggle visibility of additional map sources"))
			.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.OnGetMenuContent_Lambda([this]()
			{
				return MakeFilterMenuContent();
			})
			.ButtonContent()
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16,16))
				.Image(FAppStyle::GetBrush("Icons.Filter"))
				.ColorAndOpacity_Lambda([this]()
				{
					return HasAnyFilterEnabled()
						? FAppStyle::Get().GetSlateColor("Colors.AccentBlue")
						: FSlateColor::UseForeground();
				})
			]
		]

		+SHorizontalBox::Slot()
		.FillWidth(1)
		.Padding(4,0)
		[
			SNew(SSearchBox)
			.OnTextCommitted(this, &SCustomLaunchMapListView::OnSearchFilterTextCommitted)
			.OnTextChanged(this, &SCustomLaunchMapListView::OnSearchFilterTextChanged)
		]
	;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool SCustomLaunchMapListView::HasAnyFilterEnabled() const
{
	return MapSourceFlags != EMapSourceFlags::None;
}

TSharedRef<SWidget> SCustomLaunchMapListView::MakeFilterMenuContent()
{
	TFunction<TSharedRef<SWidget>(const FText&, EMapSourceFlags)> MakeFilterCheckBox = [this](const FText& Label, EMapSourceFlags Flag) -> TSharedRef<SWidget>
	{
		return SNew(SCheckBox)
			.IsChecked_Lambda([this, Flag]() { return EnumHasAnyFlags(MapSourceFlags, Flag) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
			.OnCheckStateChanged_Lambda([this, Flag](ECheckBoxState NewState)
			{
				if (NewState == ECheckBoxState::Checked)
				{
					MapSourceFlags |= Flag;
				}
				else
				{
					MapSourceFlags &= ~Flag;
				}
				RefreshMapList();
				OnMapFilterChanged.ExecuteIfBound(MapSourceFlags);
			})
			[
				SNew(STextBlock)
				.Text(Label)
			];
	};

	return SNew(SBox)
		.Padding(FMargin(0, 4))
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(8, 4)
			[
				MakeFilterCheckBox(LOCTEXT("FilterMyDevContent", "My Developer Content"), EMapSourceFlags::MyDeveloperContent)
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(8, 4)
			[
				MakeFilterCheckBox(LOCTEXT("FilterOtherDevContent", "Other Developers Content"), EMapSourceFlags::OtherDeveloperContent)
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(8, 4)
			[
				MakeFilterCheckBox(LOCTEXT("FilterProjectPlugins", "Project Plugins"), EMapSourceFlags::ProjectPlugins)
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(8, 4)
			[
				MakeFilterCheckBox(LOCTEXT("FilterEngineContent", "Engine Content"), EMapSourceFlags::EngineContent)
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(8, 4)
			[
				MakeFilterCheckBox(LOCTEXT("FilterEnginePlugins", "Engine Plugins"), EMapSourceFlags::EnginePlugins)
			]
		];
}

void SCustomLaunchMapListView::GetMapTreeNodeChildren(FMapTreeNodePtr Node, TArray<FMapTreeNodePtr>& OutChildren)
{
	OutChildren = Node->Children;
}



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<ITableRow> SCustomLaunchMapListView::GenerateMapTreeNodeRow(FMapTreeNodePtr Node, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(STableRow<FMapTreeNodePtr>, OwnerTable)
	[
		SNew(SCheckBox)
		.Padding(2)
		.IsChecked(this, &SCustomLaunchMapListView::GetMapTreeNodeCheckState, Node)
		.OnCheckStateChanged(this, &SCustomLaunchMapListView::SetMapTreeNodeCheckState, Node)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4,0)
			[
				SNew(SImage)
				.DesiredSizeOverride(FVector2D(16,16))
				.Image(this, &SCustomLaunchMapListView::GetMapTreeNodeIcon, Node)
				.ColorAndOpacity(this, &SCustomLaunchMapListView::GetMapTreeNodeColor, Node)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4,0)
			[
				SNew(STextBlock)
				.Text(FText::FromString(*Node->Name))
			]
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



const FSlateBrush* SCustomLaunchMapListView::GetMapTreeNodeIcon(FMapTreeNodePtr Node) const
{
	if (Node->Children.Num() == 0)
	{
		return FProjectLauncherStyle::GetBrush("Icons.Asset");
	}
	else if (MapTreeView->IsItemExpanded(Node))
	{
		return FAppStyle::GetBrush("Icons.FolderOpen");
	}
	else
	{
		return FAppStyle::GetBrush("Icons.FolderClosed");
	}
}



FSlateColor SCustomLaunchMapListView::GetMapTreeNodeColor(FMapTreeNodePtr Node) const
{
	if (Node->Children.Num() == 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentOrange");

	}
	else
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentFolder");
	}
}



ECheckBoxState SCustomLaunchMapListView::GetMapTreeNodeCheckState(FMapTreeNodePtr Node) const
{
	return Node->CheckBoxState;
}



void SCustomLaunchMapListView::SetMapTreeNodeCheckState(ECheckBoxState CheckBoxState, FMapTreeNodePtr Node)
{
	TArray<FString> CheckedMaps = SelectedMaps.Get();
	SetCheckBoxStateRecursive(Node, CheckBoxState, CheckedMaps);

	OnSelectionChanged.ExecuteIfBound(CheckedMaps);

	RefreshCheckBoxState();
}



void SCustomLaunchMapListView::OnSearchFilterTextCommitted(const FText& SearchText, ETextCommit::Type CommitType)
{
	CurrentFilterText = SearchText.ToString();
	RefreshMapList();
}



void SCustomLaunchMapListView::OnSearchFilterTextChanged(const FText& SearchText)
{
	CurrentFilterText = SearchText.ToString();
	if (CurrentFilterText.IsEmpty())
	{
		RefreshMapList();
	}
}



void SCustomLaunchMapListView::OnProjectChanged()
{
	RefreshMapList();
}



void SCustomLaunchMapListView::RefreshMapList()
{
	bMapListDirty = true;
}

void SCustomLaunchMapListView::RefreshMapListInternal()
{
	bMapListDirty = false;

	// helper to build the node tree
	MapTreeRoot = MakeShared<FMapTreeNode>();
	auto AddMapPath = [this]( const FString& Path )
	{
		// ignore this file if it isn't filtered
		if (!CurrentFilterText.IsEmpty() && !FPaths::GetBaseFilename(Path).Contains(CurrentFilterText))
		{
			return;
		}

		if (bShowFolders)
		{
			TArray<FString> PathItems;
			Path.ParseIntoArray(PathItems, TEXT("/"));

			FMapTreeNodePtr Node = MapTreeRoot;
			for (const FString& PathItem : PathItems)
			{
				if (FMapTreeNodePtr* ChildPtr = Node->Children.FindByPredicate( [PathItem]( const FMapTreeNodePtr Node ) { return Node->Name == PathItem; } ))
				{
					Node = *ChildPtr;
				}
				else
				{
					FMapTreeNodePtr Child = Node->Children.Add_GetRef(MakeShared<FMapTreeNode>());
					Child->Name = FPaths::GetBaseFilename(PathItem);
					Node = Child;
				}
			}
		}
		else
		{
			FMapTreeNodePtr Child = MapTreeRoot->Children.Add_GetRef(MakeShared<FMapTreeNode>());
			Child->Name = FPaths::GetBaseFilename(Path);
		}
	};

	// decide what to show
	bool bShowProjectMaps = !ProjectPath.Get().IsEmpty();

	TArray<FString> AvailableMaps;
	const FString WildCard = FString::Printf(TEXT("*%s"), *FPackageName::GetMapPackageExtension());

	// show maps from the project
	if (bShowProjectMaps)
	{
		FString ProjectBasePath = FPaths::GetPath(FPaths::ConvertRelativePathToFull(ProjectPath.Get()));
		FString ProjectName = FPaths::GetBaseFilename(ProjectPath.Get());

		FString ProjectContentDir = FPaths::Combine( ProjectBasePath, TEXT("Content") );
		if (FPaths::IsRelative(ProjectContentDir))
		{
			ProjectContentDir = FPaths::Combine(FPaths::RootDir(), ProjectContentDir);
		}

		// determine the developers content directory and current user's folder for filtering
		const FString DevelopersSubDir = FString(TEXT("/")) + FString(FPaths::DevelopersFolderName()) + TEXT("/");
		const FString MyDeveloperSubDir = FString(TEXT("/")) + FString(FPaths::DevelopersFolderName()) + TEXT("/") + FPaths::GameUserDeveloperFolderName() + TEXT("/");

		TArray<FString> ProjectMaps = Model->GetAvailableProjectMapPaths(ProjectBasePath);
		for (FString& ProjectMap : ProjectMaps)
		{
			// check if this map is in a developers folder
			FString RelativePath = ProjectMap;
			RelativePath.RemoveFromStart(*ProjectContentDir);
			RelativePath.ReplaceCharInline('\\', '/');

			const bool bIsInDevelopersDir = RelativePath.StartsWith(*DevelopersSubDir);
			if (bIsInDevelopersDir)
			{
				const bool bIsMyDeveloperContent = RelativePath.StartsWith(*MyDeveloperSubDir);

				if (bIsMyDeveloperContent && !EnumHasAnyFlags(MapSourceFlags, EMapSourceFlags::MyDeveloperContent))
				{
					continue;
				}

				if (!bIsMyDeveloperContent && !EnumHasAnyFlags(MapSourceFlags, EMapSourceFlags::OtherDeveloperContent))
				{
					continue;
				}
			}

			AvailableMaps.AddUnique(FPaths::GetBaseFilename(ProjectMap));

			ProjectMap.RemoveFromStart(*ProjectContentDir);
			ProjectMap.ReplaceCharInline('\\', '/');
			AddMapPath(ProjectName / ProjectMap);
		}
	}


	// show maps from the engine
	if (EnumHasAnyFlags(MapSourceFlags, EMapSourceFlags::EngineContent) || !bShowProjectMaps)
	{
		FString EngineContentDir = FPaths::ConvertRelativePathToFull(FPaths::EngineContentDir());
		TArray<FString> EngineMaps = Model->GetAvailableEngineMapPaths();
		for (FString& EngineMap : EngineMaps)
		{
			AvailableMaps.AddUnique(FPaths::GetBaseFilename(EngineMap));

			EngineMap = FPaths::ConvertRelativePathToFull(EngineMap);
			EngineMap.RemoveFromStart(*EngineContentDir);
			EngineMap.ReplaceCharInline('\\', '/');
			AddMapPath(TEXT("Engine") / EngineMap);
		}
	}

	// show maps from plugins
	if (EnumHasAnyFlags(MapSourceFlags, EMapSourceFlags::ProjectPlugins | EMapSourceFlags::EnginePlugins))
	{
		TArray<TSharedRef<IPlugin>> Plugins = IPluginManager::Get().GetEnabledPluginsWithContent();
		for (const TSharedRef<IPlugin>& Plugin : Plugins)
		{
			FString ContentDir = FPaths::ConvertRelativePathToFull(Plugin->GetContentDir());
			if (ContentDir.IsEmpty())
			{
				continue;
			}

			EPluginType PluginType = Plugin->GetType();
			const bool bIsProjectPlugin = (PluginType == EPluginType::Project || PluginType == EPluginType::Mod || PluginType == EPluginType::External);
			const bool bIsEnginePlugin = (PluginType == EPluginType::Engine || PluginType == EPluginType::Enterprise);

			if (!(bIsProjectPlugin && EnumHasAnyFlags(MapSourceFlags, EMapSourceFlags::ProjectPlugins)) &&
				!(bIsEnginePlugin && EnumHasAnyFlags(MapSourceFlags, EMapSourceFlags::EnginePlugins)))
			{
				continue;
			}

			FString PluginName = Plugin->GetFriendlyName();
			TArray<FString> PluginMaps = Model->GetAvailablePluginMapPaths(ContentDir);
			for (const FString& MapPath : PluginMaps)
			{
				AvailableMaps.AddUnique(FPaths::GetBaseFilename(MapPath));

				FString RelativeMap = FPaths::ConvertRelativePathToFull(MapPath);
				RelativeMap.RemoveFromStart(*ContentDir);
				RelativeMap.ReplaceCharInline('\\', '/');
				AddMapPath(PluginName / RelativeMap);
			}
		}
	}

	// add any maps that are selected but we have not found them
	for (const FString& CookedMap : SelectedMaps.Get() )
	{
		if (!AvailableMaps.Contains(CookedMap))
		{
			AddMapPath(TEXT("Missing") / CookedMap);
		}
	}

	RefreshCheckBoxState(true);

	MapTreeViewItemsSource = MapTreeRoot->Children;
	if (MapTreeView.IsValid())
	{
		MapTreeView->RequestTreeRefresh();
	}

}



void SCustomLaunchMapListView::RefreshCheckBoxState( bool bExpand )
{
	RefreshCheckBoxStateRecursive(MapTreeRoot, bExpand);
}



ECheckBoxState SCustomLaunchMapListView::RefreshCheckBoxStateRecursive(FMapTreeNodePtr Node, bool bExpand)
{
	// update our filter state
	Node->bFiltered = !CurrentFilterText.IsEmpty() && Node->Name.Contains(CurrentFilterText);

	// this is a map not a folder - return the check state
	if (Node->Children.Num() == 0)
	{
		return SelectedMaps.Get().Contains(Node->Name) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	// this is a folder - check all children
	ECheckBoxState LastBoxState = ECheckBoxState::Undetermined;
	for (int32 Index = 0; Index < Node->Children.Num(); Index++)
	{
		// determine the child's check state
		FMapTreeNodePtr Child = Node->Children[Index];
		Child->CheckBoxState = RefreshCheckBoxStateRecursive(Child, bExpand);

		// expand the child if necessary
		if (bExpand && (Child->bFiltered || Child->CheckBoxState != ECheckBoxState::Unchecked))
		{
			MapTreeView->SetItemExpansion(Child, true);
		}

		// our check state becomes undetermined if the child check states do not match
		if (Index > 0 && Child->CheckBoxState != LastBoxState)
		{
			LastBoxState = ECheckBoxState::Undetermined;
		}
		else
		{
			LastBoxState = Child->CheckBoxState;
		}

		// we are filtered if a child was filtered
		Node->bFiltered |= Child->bFiltered;
	}

	// return this folder's check state
	return LastBoxState;
}



void SCustomLaunchMapListView::SetCheckBoxStateRecursive(FMapTreeNodePtr Node, ECheckBoxState CheckBoxState, TArray<FString>& CheckedMaps)
{
	Node->CheckBoxState = CheckBoxState;

	if (Node->Children.Num() == 0)
	{
		if (CheckBoxState == ECheckBoxState::Checked)
		{
			CheckedMaps.Add(Node->Name);
		}
		else
		{
			CheckedMaps.Remove(Node->Name);
		}
	}
	else
	{
		for (FMapTreeNodePtr Child : Node->Children)
		{
			SetCheckBoxStateRecursive(Child, CheckBoxState, CheckedMaps);
		}
	}
}


void SCustomLaunchMapListView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bMapListDirty && bHasPaintedThisFrame)
	{
		RefreshMapListInternal();
	}

	bHasPaintedThisFrame = false;
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

int32 SCustomLaunchMapListView::OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	bHasPaintedThisFrame = true;
    return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}




#undef LOCTEXT_NAMESPACE
