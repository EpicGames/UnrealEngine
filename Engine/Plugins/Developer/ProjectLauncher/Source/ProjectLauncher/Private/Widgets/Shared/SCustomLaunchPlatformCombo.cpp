// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchPlatformCombo.h"

#include "SlateOptMacros.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSeparator.h"
#include "PlatformInfo.h"


#define LOCTEXT_NAMESPACE "SCustomLaunchPlatformCombo"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchPlatformCombo::Construct(const FArguments& InArgs)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	SelectedPlatforms = InArgs._SelectedPlatforms;
	GetAdvancedPlatformsOption = InArgs._GetAdvancedPlatformsOption;
	SupportedBuildTargets = InArgs._SupportedBuildTargets;


	FSlateFontInfo Font = InArgs._Font.IsSet() ? InArgs._Font.Get() : InArgs._TextStyle->Font;

	ChildSlot
	[
		SAssignNew(PlatformsComboBox, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&PlatformsList)
		.OnGenerateWidget( this, &SCustomLaunchPlatformCombo::OnGeneratePlatformListWidget )
		.OnSelectionChanged( this, &SCustomLaunchPlatformCombo::OnPlatformSelectionChanged )
		.MaxListHeight(1024)
		[
			SNew(SHorizontalBox)
					
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4,0, 4, 0)
			[
				SNew(SImage)
				.Visibility_Lambda([this]() { return (SelectedPlatforms.Get().Num() <= 1) ? EVisibility::Visible : EVisibility::Collapsed; } )
				.DesiredSizeOverride(FVector2D(16,16))
				.Image(this, &SCustomLaunchPlatformCombo::GetSelectedPlatformBrush)
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(this, &SCustomLaunchPlatformCombo::GetSelectedPlatformName)
				.Font(Font)
			]
		]
	];

	RefreshPlatformsList();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchPlatformCombo::OnGeneratePlatformListWidget( TSharedPtr<FString> Platform )
{
	const PlatformInfo::FTargetPlatformInfo* PlatformInfo = (!Platform.IsValid() || Platform->IsEmpty()) ? nullptr : PlatformInfo::FindPlatformInfo(FName(*Platform));
	if (PlatformInfo != nullptr)
	{
		bool bIsGroupStart = BuildTargetGroupStarts.Contains(*Platform);
		int32 GroupPadding = bIsGroupStart ? 16 : 0;
		int32 Indent = PlatformInfo->IsVanilla() ? 0 : 16;

		return SNew(SBox)
			.Padding(0,GroupPadding,0,0)
			[			
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.Visibility_Lambda( [this]() { return GetAdvancedPlatformsOption.Get() ? EVisibility::Visible : EVisibility::Collapsed; } )
					.IsChecked( this, &SCustomLaunchPlatformCombo::GetPlatformCheckState, Platform )
					.OnCheckStateChanged( this, &SCustomLaunchPlatformCombo::SetPlatformCheckState, Platform )
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4+Indent,2,4,2)
				[
					SNew(SImage)
					.DesiredSizeOverride(FVector2D(16,16))
					.Image(FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal)))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2,2,2,2)
				[
					SNew(STextBlock)
					.Text(PlatformInfo->DisplayName)
					.ColorAndOpacity_Lambda( [this, PlatformInfo] { return SCustomLaunchPlatformCombo::GetPlatformTextColor(PlatformInfo); } )
				]
			]
		;

	}
	else
	{
		return SNullWidget::NullWidget;
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



void SCustomLaunchPlatformCombo::OnPlatformSelectionChanged( TSharedPtr<FString> Platform, ESelectInfo::Type InSelectInfo )
{
	if (!Platform.IsValid())
	{
		return;
	}

	TArray<FString> Platforms;
	if (!Platform->IsEmpty())
	{
		Platforms.Add(*Platform);
	}

	OnSelectionChanged.ExecuteIfBound(Platforms);
}



const FSlateBrush* SCustomLaunchPlatformCombo::GetSelectedPlatformBrush() const
{
	const TArray<FString>& Platforms = SelectedPlatforms.Get();

	if (Platforms.Num() == 1)
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(Platforms[0]));
		if (PlatformInfo != nullptr)
		{
			return FAppStyle::GetBrush(PlatformInfo->GetIconStyleName(EPlatformIconSize::Normal));
		}
	}
	else if (Platforms.Num() == 0)
	{
		return FAppStyle::Get().GetBrush("Icons.WarningWithColor");
	}

	return FStyleDefaults::GetNoBrush();
}



FText SCustomLaunchPlatformCombo::GetSelectedPlatformName() const
{
	const TArray<FString>& Platforms = SelectedPlatforms.Get();

	if (Platforms.Num() == 1)
	{
		const PlatformInfo::FTargetPlatformInfo* PlatformInfo = PlatformInfo::FindPlatformInfo(FName(Platforms[0]));
		if (PlatformInfo != nullptr)
		{
			return PlatformInfo->DisplayName;
		}
	}
	else if (Platforms.Num() > 1)
	{
		return FText::Format( LOCTEXT("MultiplePlatforms", "Multiple Platforms ({0})"), FText::FromString(FString::Join(Platforms, TEXT(", "))));
	}

	return LOCTEXT("NoPlatform", "(no platform)");
}


ECheckBoxState SCustomLaunchPlatformCombo::GetPlatformCheckState( const TSharedPtr<FString> Platform ) const
{
	if (!Platform.IsValid() || Platform->IsEmpty())
	{
		return ECheckBoxState::Unchecked;
	}

	const TArray<FString>& Platforms = SelectedPlatforms.Get();
	return Platforms.Contains(*Platform) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SCustomLaunchPlatformCombo::SetPlatformCheckState( ECheckBoxState CheckState, TSharedPtr<FString> Platform )
{
	if (!Platform.IsValid() || Platform->IsEmpty())
	{
		return;
	}

	TArray<FString> Platforms = SelectedPlatforms.Get();
	if (CheckState == ECheckBoxState::Checked)
	{
		Platforms.AddUnique(*Platform);
	}
	else
	{
		Platforms.Remove(*Platform);
	}

	OnSelectionChanged.ExecuteIfBound(Platforms);
}

FSlateColor SCustomLaunchPlatformCombo::GetPlatformTextColor( const PlatformInfo::FTargetPlatformInfo* PlatformInfo )
{
	TSet<EBuildTargetType> BuildTargets = SupportedBuildTargets.Get();
	if (PlatformInfo != nullptr && BuildTargets.Num() > 0)
	{
		bool bHasSupportedBuildTarget;

		bHasSupportedBuildTarget = BuildTargets.Contains(PlatformInfo->PlatformType);
		if (!GetAdvancedPlatformsOption.Get() && !bHasSupportedBuildTarget)
		{
			bHasSupportedBuildTarget = 
				PlatformInfo->VanillaInfo->Flavors.ContainsByPredicate( [BuildTargets]( const PlatformInfo::FTargetPlatformInfo* Info )
				{
					return BuildTargets.Contains(Info->PlatformType);
				});
		}

		if (!bHasSupportedBuildTarget)
		{
			return FSlateColor::UseSubduedForeground();
		}
	}


	return FSlateColor::UseForeground();
}

void SCustomLaunchPlatformCombo::RefreshPlatformsList()
{
	PlatformsList.Reset();
	BuildTargetGroupStarts.Reset();

	if (GetAdvancedPlatformsOption.Get())
	{
		// sort by Client/Server, then by IniPlatformName, then by Flavor
		TArray<PlatformInfo::FTargetPlatformInfo*> SortedPlatformInfos = PlatformInfo::GetPlatformInfoArray();
		SortedPlatformInfos.Sort( [](const PlatformInfo::FTargetPlatformInfo& A, const PlatformInfo::FTargetPlatformInfo& B)
		{
			if (A.PlatformType != B.PlatformType)
			{
				return (int)A.PlatformType < (int)B.PlatformType;
			}

			if (A.IniPlatformName != B.IniPlatformName)
			{
				return A.IniPlatformName.ToString() < B.IniPlatformName.ToString();
			}

			if (A.IsVanilla() != B.IsVanilla())
			{
				return A.IsVanilla();
			}

			return A.Name.ToString() < B.Name.ToString(); // fallback - should not get here
		});


		EBuildTargetType PrevPlatformType = (SortedPlatformInfos.Num() > 0) ? SortedPlatformInfos[0]->PlatformType : EBuildTargetType::Unknown;
		for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : SortedPlatformInfos)
		{
			if (PlatformInfo->PlatformType == EBuildTargetType::Program || PlatformInfo->PlatformType == EBuildTargetType::Editor)
			{
				continue;
			}


			FString PlatformName = PlatformInfo->Name.ToString();

			PlatformsList.Add(MakeShared<FString>(PlatformName));

			// take note of which platform starts a new build type group so we can add some separation between them in the list
			if (PlatformInfo->PlatformType != PrevPlatformType)
			{
				BuildTargetGroupStarts.Add(PlatformName);
				PrevPlatformType = PlatformInfo->PlatformType;
			}
		}

	}
	else
	{
		for (const PlatformInfo::FTargetPlatformInfo* PlatformInfo : PlatformInfo::GetPlatformInfoArray())
		{
			if( PlatformInfo->PlatformType != EBuildTargetType::Game)
			{
				continue;
			}

			PlatformsList.Add(MakeShared<FString>(PlatformInfo->Name.ToString()));
		}
	}

	PlatformsComboBox->RefreshOptions();
}




#undef LOCTEXT_NAMESPACE
