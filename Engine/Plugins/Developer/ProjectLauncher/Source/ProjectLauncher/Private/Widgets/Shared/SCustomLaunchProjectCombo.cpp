// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Shared/SCustomLaunchProjectCombo.h"

#include "SlateOptMacros.h"
#include "Widgets/Input/SComboButton.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Model/ProjectLauncherModel.h"
#include "GameProjectHelper.h"
#include "Misc/Paths.h"


#define LOCTEXT_NAMESPACE "SCustomLaunchProjectCombo"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SCustomLaunchProjectCombo::Construct(const FArguments& InArgs)
{
	OnSelectionChanged = InArgs._OnSelectionChanged;
	SelectedProject = InArgs._SelectedProject;
	HasProject = InArgs._HasProject;
	bShowAnyProjectOption = InArgs._ShowAnyProjectOption;
	CurrentProjectOption = InArgs._CurrentProjectOption;

	FSlateFontInfo Font = InArgs._Font.IsSet() ? InArgs._Font.Get() : InArgs._TextStyle->Font;

	ChildSlot
	[
		SNew(SComboButton)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(this, &SCustomLaunchProjectCombo::GetProjectName)
			.Font(Font)
		]
		.MenuContent()
		[
			MakeProjectSelectionWidget()
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION



BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SWidget> SCustomLaunchProjectCombo::MakeProjectSelectionWidget()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	{
		// any project option
		if (bShowAnyProjectOption)
		{
			FUIAction AnyProjectAction(FExecuteAction::CreateSP(this, &SCustomLaunchProjectCombo::SetProjectPath, FString()));
			MenuBuilder.AddMenuEntry(LOCTEXT("AnyProjectAction", "Any Project"), LOCTEXT("AnyProjectActionHint", "This profile can be used on any project. Build target selection will not be available."), FSlateIcon(), AnyProjectAction);
			MenuBuilder.AddMenuSeparator();
		}

		// current project first
		FString ProjectName;
		if (CurrentProjectOption != ECurrentProjectOption::None)
		{
			FString ProjectPath;
			if (FPaths::IsProjectFilePathSet())
			{
				ProjectName = FPaths::GetBaseFilename(FPaths::GetProjectFilePath());
				if (CurrentProjectOption == ECurrentProjectOption::ActualProject)
				{
					ProjectPath = FPaths::GetProjectFilePath();
				}
			}

			if (ProjectName.IsEmpty() && GIsEditor)
			{
				FUIAction CurrentProjectAction(FExecuteAction::CreateSP(this, &SCustomLaunchProjectCombo::SetProjectPath, FString()));
				MenuBuilder.AddMenuEntry(LOCTEXT("CurrentProjectAction", "Current Project"), LOCTEXT("CurrentProjectHint", "Use the current project"), FSlateIcon(), CurrentProjectAction);
			}
			else if (CurrentProjectOption == ECurrentProjectOption::Empty && !ProjectName.IsEmpty() )
			{
				FUIAction CurrentProjectAction(FExecuteAction::CreateSP(this, &SCustomLaunchProjectCombo::SetProjectPath, FString()));
				MenuBuilder.AddMenuEntry(FText::Format(LOCTEXT("CurrentProjectActionFmt", "Current Project ({0})"), FText::FromString(ProjectName)), LOCTEXT("CurrentProjectHint", "Use the current project"), FSlateIcon(), CurrentProjectAction);
			}
			else if (!ProjectName.IsEmpty())
			{
				FUIAction CurrentProjectAction(FExecuteAction::CreateSP(this, &SCustomLaunchProjectCombo::SetProjectPath, ProjectPath));
				MenuBuilder.AddMenuEntry(FText::Format(LOCTEXT("CurrentProjectActionFmt", "Current Project ({0})"), FText::FromString(ProjectName)), LOCTEXT("CurrentProjectHint", "Use the current project"), FSlateIcon(), CurrentProjectAction);
			}
		}

		// add other top-level projects, ignoring the current project if we've shown it already
		const TArray<FString>& AvailableGames = FGameProjectHelper::GetAvailableGames();
		for (int32 Index = 0; Index < AvailableGames.Num(); ++Index)
		{
			FString ProjectPath = FPaths::RootDir() / AvailableGames[Index] / AvailableGames[Index] + TEXT(".uproject");
			if (CurrentProjectOption == ECurrentProjectOption::ActualProject && !ProjectName.IsEmpty() && AvailableGames[Index] == ProjectName)
			{
				continue;
			}

			FUIAction ProjectAction(FExecuteAction::CreateSP(this, &SCustomLaunchProjectCombo::SetProjectPath, ProjectPath));
			MenuBuilder.AddMenuEntry(FText::FromString(AvailableGames[Index]), FText::FromString(ProjectPath), FSlateIcon(), ProjectAction);
		}

		// browse option at the bottom
		MenuBuilder.AddMenuSeparator();
		FUIAction BrowseAction(FExecuteAction::CreateSP(this, &SCustomLaunchProjectCombo::OnBrowseForProject));
		MenuBuilder.AddMenuEntry(LOCTEXT("BrowseAction", "Browse..."), LOCTEXT("BrowseActionHint", "Browse for a project on your computer"), FSlateIcon(), BrowseAction);
	}

	return MenuBuilder.MakeWidget();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


void SCustomLaunchProjectCombo::OnBrowseForProject()
{
	FString DefaultPath = FPaths::RootDir();

	const void* ParentWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
		
	TArray<FString> OutFiles;
	if (FDesktopPlatformModule::Get()->OpenFileDialog(ParentWindowHandle, LOCTEXT("SelectProjectDialogTitle", "Select a project").ToString(), DefaultPath, TEXT(""), TEXT("Project files (*.uproject)|*.uproject"), EFileDialogFlags::None, OutFiles))
	{
		SetProjectPath(OutFiles[0]);
	}
}

FText SCustomLaunchProjectCombo::GetProjectName() const
{
	FString ProjectPath = SelectedProject.Get();

	if (!ProjectPath.IsEmpty() && (HasProject.Get() || CurrentProjectOption == ECurrentProjectOption::None))
	{
		return FText::FromString(FPaths::GetBaseFilename(ProjectPath));
	}

	if (!ProjectPath.IsEmpty() && CurrentProjectOption == ECurrentProjectOption::Empty)
	{
		return LOCTEXT("CurrentProjectAction", "Current Project");
	}
	else if (ProjectPath.IsEmpty() && bShowAnyProjectOption)
	{
		return LOCTEXT("AnyProjectAction", "Any Project");
	}
	else
	{
		return LOCTEXT("SelectProjectAction", "Select...");
	}
}

void SCustomLaunchProjectCombo::SetProjectPath(FString ProjectPath)
{
	OnSelectionChanged.ExecuteIfBound(ProjectPath);
}


#undef LOCTEXT_NAMESPACE
