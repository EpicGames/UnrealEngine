// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ProjectLauncher : ModuleRules
{
	public ProjectLauncher(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
		[
			"Core",
			"CoreUObject",
			"TargetPlatform",
			"DesktopPlatform",
			"ApplicationCore",
			"InputCore",
			"WorkspaceMenuStructure",
			"ToolWidgets",
			"TargetDeviceServices",
			"Projects",
			"DeveloperToolSettings",
			"Zen",
			"Json",
		]);

		PublicDependencyModuleNames.AddRange(
		[
			"Slate",
			"SlateCore",
			"LauncherServices",
		]);

		if (Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.AddRange(
			[
				"Engine",
				"AssetRegistry",
			]);
		}
	}
}
