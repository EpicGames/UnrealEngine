// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/UATCommandLaunchExtension.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	/**
	 * Helper base class for an automated test launch extension
	 * Creates an item in the extension menu that will toggle the test on and off
	 * note: It is not necessary for an automated test extensions to use this class
	 */
	class FAutomatedTestLaunchExtensionInstance : public FUATCommandLaunchExtensionInstanceBase
	{
	public:
		UE_API FAutomatedTestLaunchExtensionInstance( FArgs& InArgs );

	protected:
		UE_API virtual void OnAdded() override;
		UE_API virtual void OnRemoved() override;

		/**
		 * Hook to allow any modification of the automated test & profile once it has been created. At least the test name should be specified
		 */
		virtual void OnTestAdded( ILauncherProfileAutomatedTestRef InAutomatedTest ) = 0;

		/**
		 * Advanced hook to allow any modification of the profile just before the automated test is removed
		 */
		virtual void OnTestRemoved( ILauncherProfileAutomatedTestRef InAutomatedTest ) {}

		/**
		 * Access this extension's automated test if it has been created
		 */
		UE_API ILauncherProfileAutomatedTestPtr GetTest() const;
	};


	class FAutomatedTestLaunchExtension : public FUATCommandLaunchExtensionBase
	{
	public:
		UE_API virtual void GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const override;
		UE_API virtual FSlateIcon GetIcon() const override;

	protected:
		UE_API virtual ILauncherProfileUATCommandRef CreateUATCommand(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel) const override;
	};
};

#undef UE_API
