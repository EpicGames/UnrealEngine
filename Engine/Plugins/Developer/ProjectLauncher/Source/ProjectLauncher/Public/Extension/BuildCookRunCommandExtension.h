// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/LaunchExtension.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	/**
	 * Base for a launch extension for an ILauncherProfileBuildCookRun
	 */
	class FBuildCookRunExtension : public TSharedFromThis<FBuildCookRunExtension>
	{
	public:
		typedef FLaunchExtensionInstance::EConfig EConfig;

		struct FArgs
		{
			ILauncherProfileRef Profile;
			ILauncherProfileBuildCookRunRef BuildCookRun;
			ILauncherProfileBuildCookRunRef DefaultBuildCookRun;
			TSharedRef<FModel> Model;
			TSharedRef<FLaunchExtension> Extension;
			TSharedRef<FLaunchExtensionInstance> ExtensionInstance;
			FLaunchProfileTreeData& OwnerTreeData;
		};

		UE_API FBuildCookRunExtension( const FArgs& InArgs );
		UE_API virtual ~FBuildCookRunExtension();

		/**
		 * Hook to allow the extension to add extra fields to the property editing tree, if the tree builder allows it
		 * Property tree items should be hidden until the user has selected something to make it relevent, to avoid cluttering the UI.
		 */
		virtual void CustomizeTree( FLaunchProfileTreeNode& ProfileTreeNode ) {};

		/**
		 * Create the default heading for this extension
		 */
		UE_API FLaunchProfileTreeNode& AddDefaultHeading(FLaunchProfileTreeNode& ProfileTreeNode) const;



		/**
		 * Advanced hook to allow any advanced modification of the command line when a UAT command test is launched
		 */
		virtual void CustomizeUATCommandLine( FString& InOutCommandLine ) {};

		/**
		 * Notification callback when a property has been changed in our profile
		 */
		void OnPropertyChanged() {};

		/**
		 * Notification callback when the project has been changed in our profile
		 */
		virtual void OnProjectChanged() {};

		/**
		 * Notification callback when the profile is validated
		 */
		virtual void OnValidate() {};

		/** 
		 * Get the extension we belong to
		 */
		inline TSharedRef<FLaunchExtension> GetExtension() const { return Extension; }

		/** 
		 * Get the extension instance that instantiated us
		 */
		inline TSharedRef<FLaunchExtensionInstance> GetExtensionInstance() const { return ExtensionInstance; }

	protected:
		/** 
		 * Get the profile we were instantiated for
		 */
		inline ILauncherProfileRef GetProfile() const { return Profile; }

		/** 
		 * Get the BuildCookRun we were instantiated for
		 */
		inline ILauncherProfileBuildCookRunRef GetBuildCookRun() const { return BuildCookRun; }

		/** 
		 * Get the reference default BuildCookRun
		 */
		inline ILauncherProfileBuildCookRunRef GetDefaultBuildCookRun() const { return DefaultBuildCookRun; }
		
		/** 
		 * Get the model, for general purpose helper functions
		 */
		inline TSharedRef<FModel> GetModel() const { return Model; }


		/**
		 * Read a configuration string value
		 *
		 * @param Config where the value is stored
		 * @param Name name of the value to read
		 * @param DefaultValue value to return if the value isn't found
		 * @returns the value of the property, or DefaultValue if it isn't found
		 */
		UE_API FString GetConfigString(EConfig Config, const TCHAR* Name, const TCHAR* DefaultValue = TEXT("")) const;

		/**
		 * Read a configuration bool value
		 *
		 * @param Config where the value is stored
		 * @param Name name of the value to read
		 * @param DefaultValue value to return if the value isn't found
		 * @returns the value of the property, or DefaultValue if it isn't found
		 */
		UE_API bool GetConfigBool(EConfig Config, const TCHAR* Name, bool DefaultValue = false) const;

		/**
		 * Read a configuration int value
		 *
		 * @param Config where the value is stored
		 * @param Name name of the value to read
		 * @param DefaultValue value to return if the value isn't found
		 * @returns the value of the property, or DefaultValue if it isn't found
		 */
		UE_API int32 GetConfigInteger(EConfig Config, const TCHAR* Name, int32 DefaultValue = 0) const;

		/**
		 * Read a configuration float value
		 *
		 * @param Config where the value is stored
		 * @param Name name of the value to read
		 * @param DefaultValue value to return if the value isn't found
		 * @returns the value of the property, or DefaultValue if it isn't found
		 */
		UE_API float GetConfigFloat(EConfig Config, const TCHAR* Name, float DefaultValue = 0.0f) const;

		/**
		 * Write a configuration string value
		 *
		 * @param Config where the value is stored
		 * @param Name name of the value to write
		 * @param Value value to write
		 */
		UE_API void SetConfigString(EConfig Config, const TCHAR* Name, const FString& Value) const;

		/**
		 * Write a configuration bool value
		 *
		 * @param Config where the value is stored
		 * @param Name name of the value to write
		 * @param Value value to write
		 */
		UE_API void SetConfigBool(EConfig Config, const TCHAR* Name, const bool& Value) const;

		/**
		 * Write a configuration int value
		 *
		 * @param Config where the value is stored
		 * @param Name name of the value to write
		 * @param Value value to write
		 */
		UE_API void SetConfigInteger(EConfig Config, const TCHAR* Name, const int32& Value) const;

		/**
		 * Write a configuration float value
		 *
		 * @param Config where the value is stored
		 * @param Name name of the value to write
		 * @param Value value to write
		 */
		UE_API void SetConfigFloat(EConfig Config, const TCHAR* Name, const float& Value) const;

	private:
		void OnUATCommandValidation( ILauncherProfileUATCommandRef UATCommand );
		UE_API FString GetConfigKeyName(const TCHAR* Name) const;

		ILauncherProfileRef Profile;
		ILauncherProfileBuildCookRunRef BuildCookRun;
		ILauncherProfileBuildCookRunRef DefaultBuildCookRun;
		TSharedRef<FModel> Model;
		TSharedRef<FLaunchExtension> Extension;
		TSharedRef<FLaunchExtensionInstance> ExtensionInstance;
		FLaunchProfileTreeData& OwnerTreeData;
		FPropertyChangedDelegate PropertyChangedDelegate;


	};


	/**
	 * Interface for creating & maintaining build cook run extensions
	 */
	class IBuildCookRunExtensionFactory
	{
	public:

		/**
		 * A launch extension must override this to create a new build cook run extension
		 */
		virtual TSharedRef<FBuildCookRunExtension> CreateBuildCookRunExtension( const FBuildCookRunExtension::FArgs& InArgs ) = 0;

		/**
		 * Returns whether there are any active build cook run extensions from this launch extension
		 */
		virtual bool HasActiveBuildCookRunExtension() const = 0;

		/**
		 * Customizes the build cook run extensions menu to allow extenions to be enabled & disabled 
		 */
		virtual void CustomizeBuildCookRunExtensionMenu( FMenuBuilder& InMenuBuilder, const ILauncherProfileBuildCookRunRef& InBuildCookRun ) = 0;

	protected:
		/*
		 * Returns whether build cook run extensions should be enabled by default. Typically false
		 */
		virtual bool IsBuildCookRunExtensionEnabledByDefault( const ILauncherProfileBuildCookRunRef& InBuildCookRun ) const = 0;

		/*
		 * Returns whether our build cook run extensions can be toggled on or off. Typically true
		 */
		virtual bool CanToggleBuildCookRunExtension( const ILauncherProfileBuildCookRunRef& InBuildCookRun, bool bWantToEnable ) const = 0;
	};




	class FBuildCookRunCommandExtensionInstance : public FLaunchExtensionInstance, public IBuildCookRunExtensionFactory
	{
	public:

		UE_API FBuildCookRunCommandExtensionInstance( FArgs& InArgs);
		UE_API virtual ~FBuildCookRunCommandExtensionInstance();

		UE_API virtual void InternalInitialize() override;


		UE_API virtual IBuildCookRunExtensionFactory* AsBuildCookRunFactory() override { return this; }

		UE_API bool HasActiveBuildCookRunExtension() const override;

		UE_API void CustomizeBuildCookRunExtensionMenu( FMenuBuilder& InMenuBuilder, const ILauncherProfileBuildCookRunRef& InBuildCookRun )  override;

		UE_API virtual TSharedRef<FBuildCookRunExtension> CreateBuildCookRunExtension( const FBuildCookRunExtension::FArgs& InArgs ) = 0;

		UE_API virtual void CustomizeTree( FLaunchProfileTreeData& ProfileTreeData );
		UE_API virtual void OnPropertyChanged() override;

		// fixme: ideally enumerate FLaunchExtension to find those that are BCR
		virtual bool CanBeRemoved() const override final { return true; }

	protected:
		/**
		 * Get the key name to use for reading & writing a configuration value for the given BuildCookRun, expected to be passed as Name to the GetConfigXXX/SetConfigXXX functions
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value
		 * @param BuildCookRun the build cook run command
		 * 
		 * @returns namespaced version of Name
		 */
		UE_API FString GetConfigKeyName( const TCHAR* Name, const ILauncherProfileBuildCookRunRef& BuildCookRun) const;

		UE_API virtual bool IsBuildCookRunExtensionEnabledByDefault( const ILauncherProfileBuildCookRunRef& InBuildCookRun ) const override;
		UE_API virtual bool CanToggleBuildCookRunExtension( const ILauncherProfileBuildCookRunRef& InBuildCookRun, bool bWantToEnable ) const override;
		UE_API virtual bool CustomizeUATCommandLine( const ILauncherProfileUATCommandRef& InUATCommand, FString& InOutCommandLine ) override;

		UE_API void OnUATCommandAdded(const ILauncherProfileUATCommandRef& UATCommand);
		UE_API void OnUATCommandRemoved(const ILauncherProfileUATCommandRef& UATCommand);
		UE_API TSharedRef<FBuildCookRunExtension> CreateBuildCookRunExtensionInternal( const ILauncherProfileBuildCookRunRef& InBuildCookRun );

		TMap<ILauncherProfileUATCommandPtr,TSharedRef<FBuildCookRunExtension>> BuildCookRunExtensions;

		friend class FBuildCookRunExtension;
	};



	class FBuildCookRunCommandExtension : public FLaunchExtension
	{
	public:
		virtual bool IsAlwaysCreated( ILauncherProfileRef InProfile, TSharedRef<FModel> InModel ) const override { return true; }
	};


};

#undef UE_API
