// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/UATCommandLaunchExtension.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	class FBuildCookRunLaunchExtensionInstance : public FUATCommandLaunchExtensionInstanceBase
	{
	public:
		UE_API FBuildCookRunLaunchExtensionInstance( FArgs& InArgs );
	};

	class FBuildCookRunLaunchExtension : public FUATCommandLaunchExtensionBase
	{
	public:
		UE_API virtual void GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const override;
		UE_API virtual FSlateIcon GetIcon() const override;
		UE_API virtual const TCHAR* GetInternalName() const override;
		UE_API virtual FText GetDisplayName() const override;

		UE_API virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs ) override;
		UE_API virtual bool CanBeCreated( ILauncherProfileRef InProfile, TSharedRef<FModel> InModel ) const override;

	protected:
		UE_API virtual ILauncherProfileUATCommandRef CreateUATCommand(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel) const override;
	};


};

#undef UE_API
