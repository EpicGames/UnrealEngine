// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/LaunchExtension.h"
#include "Templates/Function.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	/* common command line types */
	namespace CmdLineType
	{
		extern UE_API const TCHAR* Launch;
		extern UE_API const TCHAR* UAT;
	}


	/**
	 * Base for a command line extension
	 */
	class FCmdLineParametersExtension : public TSharedFromThis<FCmdLineParametersExtension>
	{
	public:
		struct FArgs
		{
			TSharedRef<FModel> Model;
			TSharedRef<FLaunchExtensionInstance> ExtensionInstance;
			FLaunchProfileTreeNode::FSetString SetCommandLine;
			FLaunchProfileTreeNode::FGetString GetCommandLine;
			FString TypeName;
		};

		UE_API FCmdLineParametersExtension( const FArgs& InArgs );
		UE_API virtual ~FCmdLineParametersExtension();

		/** 
		 * Get the extension we belong to
		 */
		inline TSharedRef<FLaunchExtension> GetExtension() const { return Extension; }

		/** 
		 * Get the extension instance that instantiated us
		 */
		inline TSharedRef<FLaunchExtensionInstance> GetExtensionInstance() const { return ExtensionInstance; }

	protected:
		/**
		 * Hook to allow the extension to extend the extension parameters menu
		 */
		virtual void CustomizeParametersSubmenu( FMenuBuilder& MenuBuilder ) {};

		/**
		 * Notification callback when the project has been changed in our profile
		 */
		virtual void OnProjectChanged() {};

		/** 
		 * Get the profile we were instantiated for
		 */
		inline ILauncherProfileRef GetProfile() const { return Profile; }
	
		/** 
		 * Get the model, for general purpose helper functions
		 */
		inline TSharedRef<FModel> GetModel() const { return Model; }

		/**
		 * Update the command line
		 */
		UE_API void SetCommandLine( const FString& InCommandLine );

		/**
		 * Get the current command line
		 */
		UE_API FString GetCommandLine() const;

	private:
		ILauncherProfileRef Profile;
		TSharedRef<FModel> Model;
		TSharedRef<FLaunchExtension> Extension;
		TSharedRef<FLaunchExtensionInstance> ExtensionInstance;
		FLaunchProfileTreeNode::FSetString SetCommandLineDelegate;
		FLaunchProfileTreeNode::FGetString GetCommandLineDelegate;
		FString TypeName;

		friend class ICmdLineParametersExtensionFactory;
	};


	/**
	 * Interface for creating & maintaining command line extensions
	 */
	class ICmdLineParametersExtensionFactory
	{
	public:
		virtual ~ICmdLineParametersExtensionFactory() = default;

		/**
		 * Create an instance of a command line extension. A launch extension can override this to customize the command line menu
		 */
		UE_API virtual TSharedRef<FCmdLineParametersExtension> Create( const FCmdLineParametersExtension::FArgs& InArgs );

		/**
		 * Which commandline type the extension supports. Either a custom string or one of ProjectLauncher::CmdLineType::
		 */
		virtual FString GetCmdLineType() const { return CmdLineType::Launch; }

		/**
		 * Returns the user-facing name for the given parameter. It will default to the parameter itself.
		 */
		UE_API virtual FText GetCmdLineParameterDisplayName( const FString& InParameter ) const;

		/**
		 * Returns the parameters that this extension provides. They will be added to the submenu.
		 */
		virtual void GetCmdLineParameters( TArray<FString>& OutParameters ) const = 0;

		/**
		 * Returns the parameters that will be automatically added to the command line when this extension is enabled.
		 * Not available for 'always created' extensions
		 */
		virtual void GetDefaultCmdLineParameters( TArray<FString>& OutParameters ) const {}



		/**
		 * Create the command line parameters menu for the given extension
		 */
		static UE_API void MakeCmdLineParametersSubmenu(FMenuBuilder& MenuBuilder, uint64 InstanceID, const FString& TypeName, TSharedRef<FModel> Model, TSharedRef<FLaunchExtensionInstance> Instance, const FLaunchProfileTreeNode::FSetString& SetValue, const FLaunchProfileTreeNode::FGetString& GetValue );

		/**
		 * Remove any command line extensions from the given launch extension
		 */
		static UE_API void DestroyCmdLineParametersExtensions(TSharedPtr<FLaunchExtensionInstance> Instance);

	};
};

#undef UE_API
