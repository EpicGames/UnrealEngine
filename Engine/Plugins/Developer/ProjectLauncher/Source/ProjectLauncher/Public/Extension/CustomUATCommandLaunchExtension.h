// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/UATCommandLaunchExtension.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	/**
	 * Base class for an custom UAT command launch extension
	 */
	class FCustomUATCommandLaunchExtensionInstance : public FUATCommandLaunchExtensionInstanceBase
	{
	public:
		UE_API FCustomUATCommandLaunchExtensionInstance( FArgs& InArgs );

	protected:
		UE_API virtual void OnAdded() override;
		UE_API virtual void OnRemoved() override;

		/**
		 * Hook to allow any modification of the UAT Command & profile once it has been created. At least the UAT Command name should be specified
		 */
		virtual void OnUATCommandAdded( ILauncherProfileUATCommandRef InUATCommand ) = 0;

		/**
		 * Advanced hook to allow any modification of the profile just before the UAT Command is removed
		 */
		virtual void OnUATCommandRemoved( ILauncherProfileUATCommandRef InUATCommand ) {}
	};



	class FCustomUATCommandLaunchExtension : public FUATCommandLaunchExtensionBase
	{
	public:
		UE_API virtual void GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const override;

	protected:
		UE_API virtual ILauncherProfileUATCommandRef CreateUATCommand(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel) const override;
	};

};

#undef UE_API
