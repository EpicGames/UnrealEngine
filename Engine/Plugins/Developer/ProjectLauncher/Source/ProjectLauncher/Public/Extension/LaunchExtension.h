// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "ProfileTree/LaunchProfileTreeData.h"
#include "ILauncherProfile.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Delegates/Delegate.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	class FLaunchExtension;
	class FLaunchExtensionInstance;

	typedef TMulticastDelegate<void()> FPropertyChangedDelegate;

	/**
	 * Base class for a launch extension instance.
	 * Used while editing a specific profile, and when finalizing the command line arguments during profile launch.
	 * 
	 * Created by a specialization of FLaunchExtension as follows:
	 * 
	 * TSharedPtr<ProjectLauncher:FLaunchExtensinInstance> FMyLaunchExtension::CreateInstanceForProfile( ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs )
	 * {
	 *     return MakeShared<FMyLaunchExtensionInstance>(InArgs);
	 * }
	 */
	class FLaunchExtensionInstance : public TSharedFromThis<FLaunchExtensionInstance>
	{
	public:
		struct FArgs
		{
			ILauncherProfileRef Profile;
			ILauncherProfileRef DefaultProfile;
			TSharedRef<FModel> Model;
			TSharedRef<FLaunchExtension> Extension;
			FLaunchProfileTreeData& OwnerTreeData;
			ILauncherProfileUATCommandPtr UATCommand;
		};

		UE_API FLaunchExtensionInstance( FArgs& InArgs );
		UE_API virtual ~FLaunchExtensionInstance();

		/**
		 * Returns the parameters that this extension provides. They will be added to the submenu.
		 */
		UE_DEPRECATED(5.8, "refactor code to use ICmdLineParametersExtension")
		UE_API virtual bool GetExtensionParameters( TArray<FString>& OutParameters ) const;

		/**
		 * Returns the user-facing name for the given parameter. It will default to the parameter itself.
		 */
		UE_DEPRECATED(5.8, "refactor code to use ICmdLineParametersExtension")
		UE_API virtual FText GetExtensionParameterDisplayName( const FString& InParameter ) const;

		/**
		 * Returns the user facing variables that this extension provides, in "$(name)" format
		 */
		UE_API virtual bool GetExtensionVariables( TArray<FString>& OutVariables ) const;

		/**
		 * Returns the current value for the given variable
		 */
		UE_API virtual bool GetExtensionVariableValue( const FString& InParameter, FString& OutValue ) const;

		/**
		 * Hook to allow the extension to extend the extension parameters menu
		 */
		UE_DEPRECATED(5.8, "refactor code to use ICmdLineParametersExtension")
		virtual void CustomizeParametersSubmenu( FMenuBuilder& MenuBuilder ) {};

		/**
		 * Hook to allow the extension to add extra fields to the property editing tree, if the tree builder allows it
		 * Property tree items should be hidden until the user has selected something to make it relevent, to avoid cluttering the UI.
		 */
		UE_API virtual void CustomizeTree( FLaunchProfileTreeData& ProfileTreeData );

		/**
		 * Advanced hook to allow any advanced modification of the command line when our profile is launched
		 */
		virtual void CustomizeLaunchCommandLine( FString& InOutCommandLine ) {};
		
		/**
		 * Advanced hook to allow any advanced modification of the command line when our profile is launched for the given build target type
		 */
		virtual void CustomizeTargetLaunchCommandLine( FString& InOutCommandLine, EBuildTargetType InBuildTargetType ) {};

		/**
		 * Advanced hook to allow any advanced modification of the command line when a UAT command test is launched
		 */
		UE_API virtual bool CustomizeUATCommandLine( const ILauncherProfileUATCommandRef& InUATCommand, FString& InOutCommandLine );

		/**
		 * Notification callback when a property has been changed in our profile
		 */
		UE_API virtual void OnPropertyChanged();

		/**
		 * Notification callback when the project has been changed in our profile
		 */
		virtual void OnProjectChanged() {};

		/**
		 * Notification callback when the profile is validated
		 */
		virtual void OnValidateProfile() {};

		/**
		 * Create the default heading for this extension
		 */
		virtual UE_API FLaunchProfileTreeNode& AddDefaultHeading(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData) const;

		/*
		 * Populate the custom extension menu for this extension. This menu would typically contain
		 * items that might enable bespoke tree customization options for example.
		 */
		virtual void MakeCustomExtensionSubmenu(FMenuBuilder& MenuBuilder) {}

		/*
		 * Whether the given extension can be removed or not
		 */
		virtual bool CanBeRemoved() const { return true; }

		/** 
		 * Get the extension that instantiated us
		 */
		inline TSharedRef<FLaunchExtension> GetExtension() const { return Extension; }

		/** 
		 * Get the profile we were instantiated for
		 */
		inline ILauncherProfileRef GetProfile() const { return Profile; }

		/** 
		 * Get the reference default profile
		 */
		inline ILauncherProfileRef GetDefaultProfile() const { return DefaultProfile; }

		/** 
		 * Get the model, for general purpose helper functions
		 */
		inline TSharedRef<FModel> GetModel() const { return Model; }

		/**
		 * Get the property changed delegate that will be called when an extension changes the profile
		 */
		inline FPropertyChangedDelegate& GetPropertyChangedDelegate() { return PropertyChangedDelegate; }

		/**
		 * Attempt to cast this launch extension instance to a build cook run factory
		 */
		virtual class IBuildCookRunExtensionFactory* AsBuildCookRunFactory() { return nullptr; }

		/**
		 * Attempt to cast this launch extension instance to a launch command line factory
		 */
		virtual class ICmdLineParametersExtensionFactory* AsCmdLineParametersFactory() { return nullptr; }

		/**
		 * Determine if this launch extension instance manages the given UAT command
		 */
		virtual bool ManagesUATCommand( const ILauncherProfileUATCommandRef& UATCommand ) const { return false; }

		/**
		 * Get the associated UAT command, if any
		 */
		virtual ILauncherProfileUATCommandPtr GetUATCommand() const { return nullptr; }

		/**
		 * Internal intitialization function, called immediately after construction - don't call directly
		 */
		UE_API virtual void InternalInitialize();


		/**
		 * Internal property tree customization function - don't call directly
		 */
		UE_API void CustomizeTree();

		UE_API virtual void OnAdded() {};
		UE_API virtual void OnRemoved() {};

		/**
		 * Broadcast an event to all other extension instances associated to the same profile.
		 */
		UE_API virtual void BroadcastEvent(const FString& EventName, void* EventData = nullptr);

		/**
		 * The listener checks the incoming event name and chooses to handle the event accordingly.
		 */
		UE_API virtual void HandleEventCallback(const FString& EventName, void* EventData) {}

		/**
		 * Search for the extension instances associated with the given profile by name
		 */
		UE_API static TArray<TSharedPtr<FLaunchExtensionInstance>> GetProfileExtensionInstancesByName(const FString& ExtensionName, ILauncherProfileRef InProfile);

	protected:

		UE_DEPRECATED(5.8, "Use ProjectLauncher::CmdLineUtils version")
		UE_API bool IsParameterUsed( const FString& InParameter ) const;

		UE_DEPRECATED(5.8, "Use ProjectLauncher::CmdLineUtils version")
		UE_API void SetParameterUsed( const FString& InParameter, bool bUsed );

		UE_DEPRECATED(5.8, "Use ProjectLauncher::CmdLineUtils version")
		UE_API void AddParameter( const FString& InParameter );

		UE_DEPRECATED(5.8, "Use ProjectLauncher::CmdLineUtils version")
		UE_API void RemoveParameter( const FString& InParameter );

		UE_DEPRECATED(5.8, "Use ProjectLauncher::CmdLineUtils version")
		UE_API FString GetParameterValue( const FString& InParameter ) const;

		UE_DEPRECATED(5.8, "Use ProjectLauncher::CmdLineUtils version")
		UE_API bool UpdateParameterValue( const FString& InParameter, const FString& InNewValue );

		UE_DEPRECATED(5.8, "Use ProjectLauncher::CmdLineUtils version")
		UE_API FString GetFinalParameter( const FString& InParameter ) const;

		/** 
		 * Get the current command line
		 *
		 * @returns the current command line string
		 */
		UE_DEPRECATED(5.8, "access the appropriate field from a BuildCookRun instance, or use ICmdLineParametersExtension")
		UE_API FString GetCommandLine() const;

		/** 
		 * Update the current command line
		 * 
		 * @param CommandLine the new command line
		 */
		UE_DEPRECATED(5.8, "access the appropriate field from a BuildCookRun instance, or use ICmdLineParametersExtension")
		UE_API void SetCommandLine( const FString& CommandLine );

		/**
		 * Get the owner tree data
		 */
		inline FLaunchProfileTreeData& GetOwnerTreeData() const { return OwnerTreeData; }


	public:
		/**
		 * Get the name of the platform to launch on. this is the highest priority platform/target in the list of platforms
		 */
		UE_API FName GetLaunchPlatformName() const;

		/** 
		* Enumeration to choose where a value should be stored
		*/
		enum class EConfig : uint8
		{
			User_Common,     ///< value is shared between all instances of this extension
			User_PerProfile, ///< value is specific to this profile & extension
			PerProfile,      ///< value is saved with the profile
		};

		/**
		 * Read a configuration string value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value to read
		 * @param DefaultValue value to return if the value isn't found
		 * @returns the value of the string, or DefaultValue if it isn't found
		 */
		UE_API FString GetConfigString( EConfig Config, const TCHAR* Name, const TCHAR* DefaultValue = TEXT("") ) const;

		/**
		 * Read a configuration bool value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value to read
		 * @param DefaultValue value to return if the value isn't found
		 * @returns the value of the bool, or DefaultValue if it isn't found
		 */
		UE_API bool GetConfigBool( EConfig Config, const TCHAR* Name, bool DefaultValue = false ) const;

		/**
		 * Read a configuration int value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value to read
		 * @param DefaultValue value to return if the value isn't found
		 * @returns the value of the bool, or DefaultValue if it isn't found
		 */
		UE_API int32 GetConfigInteger( EConfig Config, const TCHAR* Name, int32 DefaultValue = 0 ) const;

		/**
		 * Read a configuration float value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value to read
		 * @param DefaultValue value to return if the value isn't found
		 * @returns the value of the bool, or DefaultValue if it isn't found
		 */
		UE_API float GetConfigFloat( EConfig Config, const TCHAR* Name, float DefaultValue = 0.0f ) const;

		/**
		 * Write a configuration string value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value to write
		 * @param Value value to write
		 */
		UE_API void SetConfigString( EConfig Config, const TCHAR* Name, const FString& Value ) const;

		/**
		 * Write a configuration bool value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value to write
		 * @param Value value to write
		 */
		UE_API void SetConfigBool( EConfig Config, const TCHAR* Name, const bool& Value ) const;

		/**
		 * Write a configuration int value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value to write
		 * @param Value value to write
		 */
		UE_API void SetConfigInteger( EConfig Config, const TCHAR* Name, const int32& Value ) const;

		/**
		 * Write a configuration float value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value to write
		 * @param Value value to write
		 */
		UE_API void SetConfigFloat( EConfig Config, const TCHAR* Name, const float& Value ) const;



		/**
		 * Get the final key name to use for reading & writing a configuration value
		 * 
		 * @param Config where the value is stored
		 * @param Name name of the value
		 * 
		 * @returns namespaced version of Name
		 */
		UE_API FString GetConfigKeyName( EConfig Config, const TCHAR* Name ) const;

		/**
		 * Signal to any listeners that a property in the profile has changed
		 */
		UE_API void BroadcastPropertyChanged() const;


		/** 
		 * Request that the property tree should be refreshed - such as when a widget has changed size
		*/
		UE_API void RequestTreeRefresh();

		/** 
		 * Request that the enture property tree is rebuilt - such as when adding or removing top-level sections
		*/
		UE_API void RequestFullTreeRebuild();

	private:

		UE_API virtual void OnCustomValidation();

		UE_API virtual const TCHAR* GetInternalName() const;

		ILauncherProfileRef Profile;
		ILauncherProfileRef DefaultProfile;
		TSharedRef<FModel> Model;
		TSharedRef<FLaunchExtension> Extension;
		FLaunchProfileTreeData& OwnerTreeData;
		FPropertyChangedDelegate PropertyChangedDelegate;
	};






	/**
	 * Base class for a launch extension.
	 * 
	 * Singleton instance is registered with this plugin during initialization as follows:
	 * 
	 *   TSharedPtr<FMyLaunchExtension> MyExtension = MakeShared<FMyLaunchExtension>();
	 *   IProjectLauncherModule::Get().RegisterExtension( MyExtension.ToSharedRef() );
	 */
	class FLaunchExtension : public TSharedFromThis<FLaunchExtension>
	{
	public:
		virtual ~FLaunchExtension() = default;

		/**
		 * Returns the debug name for this extension
		 */
		virtual const TCHAR* GetInternalName() const = 0;

		/**
		 * Returns the user-facing name for this extension
		 */
		virtual FText GetDisplayName() const = 0;

		/**
		 * Returns the icon to use for this extension in menus etc
		 */
		UE_API virtual FSlateIcon GetIcon() const;

		/**
		 * Whether this extension is always created. the user cannot remove it
		 */
		virtual bool IsAlwaysCreated( ILauncherProfileRef InProfile, TSharedRef<FModel> InModel ) const { return false; }

		/**
		 * Whether this extension will be created by default. the user can still remove it
		 */
		virtual bool IsCreatedByDefault( ILauncherProfileRef InProfile, TSharedRef<FModel> InModel ) const { return false; }

		/**
		 * Whether this extension manages a UAT command
		 */
		virtual bool IsUATCommandManager() const { return false; }

		/**
		 * Whether this extension can be created at this time
		 */
		virtual bool CanBeCreated( ILauncherProfileRef InProfile, TSharedRef<FModel> InModel ) const { return true; }


		/**
		 * Attempt to cast this launch extension to an BuildCookRun launch extension (this is an extension for a build cook run UAT command)
		 */
		virtual class FBuildCookRunLaunchExtension* AsBuildCookRunLaunchExtension() { return nullptr; }

		/**
		 * Attempt to cast this launch extension instance to a UAT command factory
		 */
		virtual class IUATCommandExtensionFactory* AsUATCommandFactory() { return nullptr; }

		/**
		 * Instantiate all compatible extensions
		 * 
		 * @param InProfile the current profile 
		 * @param InModel helper class
		 * @param InOwnerTreeData the property tree data associated with the profile
		 * @returns array of all compatible extensions
		 */
		UE_API static TArray<TSharedPtr<FLaunchExtensionInstance>> CreateExtensionInstancesForProfile( ILauncherProfileRef InProfile, TSharedRef<FModel> InModel, FLaunchProfileTreeData& InOwnerTreeData );

		UE_API static TSharedPtr<FLaunchExtensionInstance> CreateExtensionInstance( TSharedRef<FLaunchExtension> InExtension, ILauncherProfileRef InProfile, TSharedRef<FModel> InModel, FLaunchProfileTreeData& InOwnerTreeData, ILauncherProfileUATCommandPtr InUATCommand = nullptr );
		UE_API static void DestroyExtensionInstance(TSharedRef<FLaunchExtensionInstance> InExtensionInstance, ILauncherProfileRef InProfile, TSharedRef<FModel> InModel);

		UE_API static TSharedPtr<FLaunchExtensionInstance> GetExtensionInstance(TSharedRef<FLaunchExtension> InExtension, ILauncherProfileRef InProfile);
		UE_API static TArray<TSharedRef<FLaunchExtensionInstance>> GetExtensionInstances(TSharedRef<FLaunchExtension> InExtension, ILauncherProfileRef InProfile);

		UE_API static TArray<TSharedRef<FLaunchExtension>> GetExtensions();


		/**
		 * Describes how this extension is represented in the extensions menu
		 */
		struct FExtensionsMenuEntry
		{
			// common menu locations
			static UE_API const FExtensionsMenuEntry None;
			static UE_API const FExtensionsMenuEntry Default;
			static UE_API const FExtensionsMenuEntry OwnSection;
			static UE_API const FExtensionsMenuEntry Deprecated;
			static UE_API const FExtensionsMenuEntry AutomatedTests;
			static UE_API const FExtensionsMenuEntry UATCommands;

			// internal names
			FString SectionName; // if this is empty, the extension will be placed in its own section
			FString SubmenuName;

			// friendly names
			FText SectionDisplayName;
			FText SubmenuDisplayName;

			// type of menu entry
			enum EType
			{
				Type_None,			// does not appear in the extensions menu
				Type_Section,		// appears in the extensions menu in the common section.
				Type_SubMenu,		// appears in the extensions menu in a submenu of the SectionName section
			};
			EType Type = Type_None;

			bool bIsDefault = false; // default sections should be kept at the top
		};
		UE_API virtual void GetExtensionsMenuEntry( FExtensionsMenuEntry& MenuEntry ) const;

		virtual void MakeCustomExtensionSubmenu(FMenuBuilder& MenuBuilder, ILauncherProfileRef InProfile, TSharedRef<FModel> InModel) {}

		/**
		 * Destroy all extension instances for the given profile
		 * 
		 * @param InProfile the profile who's extension instances to return
		 */
		UE_API static void DestroyExtensionInstancesForProfile(ILauncherProfilePtr InProfile);

	private:
		/**
		 * Create an instance of the launch extension for the given profile - called internally by CreateExtensionInstancesForProfile
		 *
		 * @returns: new instance, or null if it isn't appropriate
		 */
		virtual TSharedPtr<FLaunchExtensionInstance> CreateInstanceForProfile( FLaunchExtensionInstance::FArgs& InArgs ) = 0;
	};
};

#undef UE_API
