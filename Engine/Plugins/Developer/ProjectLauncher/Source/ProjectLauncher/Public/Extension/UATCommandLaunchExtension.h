// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/LaunchExtension.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	/**
	 * Helper base class for an extension that is tied to a UAT command in the profile
	 */
	class FUATCommandLaunchExtensionInstanceBase : public FLaunchExtensionInstance
	{
	public:
		UE_API FUATCommandLaunchExtensionInstanceBase( FArgs& InArgs );

		UE_API virtual bool ManagesUATCommand( const ILauncherProfileUATCommandRef& InUATCommand ) const override;
		UE_API virtual ILauncherProfileUATCommandPtr GetUATCommand() const override;
		
		/**
		 * Advanced hook to allow any advanced modification of the command line when this extension's UAT Command is launched
		 */
		virtual void CustomizeUATCommandLine( FString& InOutCommandLine ) {}

	protected:
		const ILauncherProfileUATCommandRef UATCommand;

		UE_API virtual bool CustomizeUATCommandLine( const ILauncherProfileUATCommandRef& InUATCommand, FString& InOutCommandLine ) override;
		UE_API virtual void OnCustomValidation() override;
		UE_API virtual FLaunchProfileTreeNode& AddDefaultHeading(FLaunchProfileTreeData& ProfileTreeData) const override;
		UE_API virtual const TCHAR* GetInternalName() const override;
	};

	/**
	 * Interface for creating & maintaining UAT command extensions
	 */
	class IUATCommandExtensionFactory
	{
	public:
		virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateUATCommandExtensionInstance(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel, FLaunchProfileTreeData& InOwnerTreeData) = 0;

	protected:
		virtual ILauncherProfileUATCommandRef CreateUATCommand(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel) const = 0;

	};


	class FUATCommandLaunchExtensionBase : public FLaunchExtension, public IUATCommandExtensionFactory
	{
	public:
		virtual bool IsUATCommandManager() const { return true; }
		
		virtual IUATCommandExtensionFactory* AsUATCommandFactory() override { return this; }

	protected:
		UE_API virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateUATCommandExtensionInstance(ILauncherProfileRef InProfile, TSharedRef<FModel> InModel, FLaunchProfileTreeData& InOwnerTreeData) override;

	};

};

#undef UE_API
