// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "ILauncherProfileManager.h"
#include "ILauncherWorker.h"
#include "Templates/SharedPointer.h"
#include "Misc/DataDrivenPlatformInfoRegistry.h"
#include "HAL/Event.h"

#define UE_API PROJECTLAUNCHER_API



namespace PlatformInfo
{
	struct FTargetPlatformInfo;
};

class ITargetDeviceProxyManager;
class ITargetDeviceProxy;
class ILauncher;
struct FTargetInfo;

namespace ProjectLauncher
{
	extern const FText ProjectLauncherText;
	extern bool bUseClassicView;

	/* Defines how content is stored and deployed for a profile */
	enum class EContentScheme : uint8
	{
		PakFiles,
		ZenStreaming,
		ZenPakStreaming,
		DevelopmentPackage,
		SubmissionPackage,
		LooseFiles,
		CookOnTheFly,
		PreStagedBuild,
		MAX
	};

	/* Returns all possible content schemes */
	TArray<EContentScheme> GetAllContentSchemes();

	/* Get user-visible display name for a content scheme */
	FText GetContentSchemeDisplayName(EContentScheme ContentScheme);

	/* Get tooltip text describing a content scheme */
	FText GetContentSchemeToolTip(EContentScheme ContentScheme);

	/* Returns a launch error message for a given profile, optionally with annotations */
	FText GetProfileLaunchErrorMessage(ILauncherProfilePtr Profile, bool bWithAnnotations = false );

	/* Convert a content scheme to string */
	const TCHAR* LexToString( const ProjectLauncher::EContentScheme& ContentScheme);

	/* Try to parse a string into a content scheme enum value */
	bool LexTryParseString( ProjectLauncher::EContentScheme& OutContentScheme, const TCHAR* String );

	/* Message box helper */
	UE_API bool GetUserConfirmation( const FText& Message, const FText& Title = ProjectLauncherText, bool bDefaultValue = true );

	/* Defines the type of a profile (basic, custom, or advanced) */
	enum class EProfileType : uint8
	{
		Invalid,
		Basic,
		Custom,
		Advanced,
	};

	/* Holds a single log message emitted during launch */
	struct FLaunchLogMessage
	{
		TSharedRef<FString> Message;
		ELogVerbosity::Type Verbosity;

		FLaunchLogMessage(const FString& InMessage, ELogVerbosity::Type InVerbosity)
			: Message(MakeShared<FString>(InMessage))
			, Verbosity(InVerbosity)
		{}
	};

	/* Cache of project settings relevant for launch profiles */
	struct FProjectSettings
	{
		// whether this project is the one that is currently open in the editor
		bool bIsCurrentEditorProject = false;

		// whether Zen Store is enabled via the project settings - in this case, we can't use Loose Files
		bool bUseZenStore = false;

		// whether Io Store is enabled via the project settings - we can only use zen workflows when this is enabled
		bool bUseIoStore = false;

		// whether a workspace will be created automatically when using Zen Pak streaming
		bool bHasAutomaticZenPakStreamingWorkspaceCreation = false;

		// whether Zen Server is allowed to accept connections from remote machines (i.e. console devkits, phones etc)
		bool bAllowRemoteNetworkService = false;

		// default build target names
		FString DefaultBuildTargetName;
		FString DefaultServerTargetName;
		FString DefaultClientTargetName;
		FString DefaultEditorTargetName;

		// fallback "default" build target to use if none is specified
		FString FallbackDefaultBuildTarget;

		// cached project config ini
		FConfigCacheIni* Config = nullptr;

		// whether this is still being initialized
		FEvent* Pending = nullptr;
		bool bIsReady = false;
	};


	/* Delegate fired when a profile is clicked in the UI */
	DECLARE_DELEGATE_OneParam(FOnProfileClicked, const ILauncherProfilePtr&)

	/* Delegate fired when the selected profile changes */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnSelectedProfileChanged, const ILauncherProfilePtr& /*new*/, const ILauncherProfilePtr& /*old*/)

	/* Delegate fired when project settings have been loaded */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnProjectSettingsReady, const FString& /*ProjectPath*/)

	/* Delegate fired when the selected platform changes */
	DECLARE_MULTICAST_DELEGATE(FOnSelectedPlatformChanged)


	/* common command line types */
	namespace CmdLineType
	{
		extern UE_API const TCHAR* Launch;
		extern UE_API const TCHAR* UAT;
	}


	/*
	 * This class contains a selection of helper functions and the current profile state
	 */
	class FModel : public TSharedFromThis<FModel>
	{
	public:
		/* Constructor/destructor/one-time init */
		UE_API FModel(const TSharedRef<ITargetDeviceProxyManager>& InDeviceProxyManager, const TSharedRef<ILauncher>& InLauncher, const TSharedRef<ILauncherProfileManager>& InProfileManager);
		UE_API ~FModel();
		UE_API void Init();

		/* Get the target device proxy manager */
		const TSharedRef<ITargetDeviceProxyManager>& GetDeviceProxyManager() const
		{
			return DeviceProxyManager;
		}

		/* Get the launcher instance */
		const TSharedRef<ILauncher>& GetLauncher() const
		{
			return Launcher;
		}

		/* Get the launcher profile manager */
		const TSharedRef<ILauncherProfileManager>& GetProfileManager() const
		{
			return ProfileManager;
		}

		/* Get all known launcher profiles */
		const TArray<ILauncherProfilePtr>& GetAllProfiles() const
		{
			return AllProfiles;
		}

		/* Get the basic launch profile */
		ILauncherProfilePtr GetBasicLaunchProfile() const
		{
			return BasicLaunchProfile;
		}

		/* Get the currently selected profile */
		const ILauncherProfilePtr& GetSelectedProfile() const
		{
			return SelectedProfile;
		}

		/* Delegate fired when a profile is selected */
		FOnSelectedProfileChanged& OnProfileSelected()
		{
			return ProfileSelectedDelegate;
		}

		/* Change the selected profile */
		UE_API void SelectProfile(const ILauncherProfilePtr& Profile);

		/* Sort all profiles alphabetically, keeping the basic profile first in the list */
		UE_API void SortProfiles();

		/* Get the type of a profile (basic, custom, advanced) */
		UE_API EProfileType GetProfileType(const ILauncherProfileRef& Profile) const;

		/* Returns true if the given profile is advanced (multiple platforms/cultures, DLC, patches, etc) */
		UE_API bool IsAdvancedProfile(const ILauncherProfileRef& Profile) const;

		/* Returns true if the given profile is the basic launch profile */
		UE_API bool IsBasicLaunchProfile(const ILauncherProfilePtr& Profile) const;

		/* Create a new custom profile with the given name */
		UE_API ILauncherProfileRef CreateCustomProfile(const TCHAR* Name);

		/* Create a new basic launch profile */
		UE_API ILauncherProfileRef CreateBasicLaunchProfile();

		/* Clone an existing custom profile */
		UE_API ILauncherProfilePtr CloneCustomProfile(const ILauncherProfileRef& Profile);

		/* Get the default basic launch profile - used for "reset to default"  */
		UE_API const ILauncherProfileRef GetDefaultBasicLaunchProfile() const;

		/* Get the default custom launch profile - used for "reset to default" */
		UE_API const ILauncherProfileRef GetDefaultCustomLaunchProfile() const;

		/* Get the default build cook run for the basic profile - used for "reset to default" */
		UE_API const ILauncherProfileBuildCookRunRef GetBasicDefaultBuildCookRun() const;

		/* Get the default build cook run for the custom profile - used for "reset to default" */
		UE_API const ILauncherProfileBuildCookRunRef GetCustomDefaultBuildCookRun() const;


		/* Determine the content scheme for a profile */
		UE_API EContentScheme DetermineProfileContentScheme(const ILauncherProfileBuildCookRunRef& BuildCookRun) const;

		/* Apply a content scheme to a profile, setting cook/deploy/packaging options etc */
		UE_API void SetProfileContentScheme(EContentScheme ContentScheme, const FProjectSettings& ProjectSettings, 
			const ILauncherProfileBuildCookRunRef& BuildCookRun, bool bWantToCook = true,
			ELauncherProfileDeploymentModes::Type DefaultDeploymentMode = ELauncherProfileDeploymentModes::CopyToDevice);

		/* Record a new log message from launching */
		UE_API TSharedPtr<FLaunchLogMessage> AddLogMessage(const FString& InMessage, ELogVerbosity::Type InVerbosity = ELogVerbosity::Log);

		/* Clear all log messages */
		UE_API void ClearLogMessages();
		
		/* Get number of log messages */
		int32 GetNumLogMessages() const
		{
			return LaunchLogMessages.Num();
		}

		/* Get platform info for a profile or platform name */
		UE_DEPRECATED(5.8, "Use GetPlatformInfos")
		static UE_API const PlatformInfo::FTargetPlatformInfo* GetPlatformInfo(const ILauncherProfilePtr& Profile);
		static UE_API const PlatformInfo::FTargetPlatformInfo* GetPlatformInfo(FName PlatformName, const FTargetInfo& BuildTargetInfo);
		static UE_API TArray<const PlatformInfo::FTargetPlatformInfo*> GetPlatformInfos(const ILauncherProfileBuildCookRunRef& BuildCookRun);
		static UE_API TArray<const PlatformInfo::FTargetPlatformInfo*> GetPlatformInfosForAllCommands(const ILauncherProfilePtr& Profile);

		/* Returns true if profile or platform name refers to host platform */
		UE_DEPRECATED(5.8, "No longer in use. IsUsingRemotePlatform may be of use")
		static UE_API bool IsHostPlatform(const ILauncherProfilePtr& Profile);
		static UE_API bool IsHostPlatform(FName PlatformName);
		static UE_API bool IsUsingRemotePlatform(const ILauncherProfileBuildCookRunRef& BuildCookRun);

		/* Get build target info for a project path or profile */
		UE_API FTargetInfo GetBuildTargetInfo(FString BuildTargetName, const FString& ProjectPath);
		UE_DEPRECATED(5.8, "Use GetBuildTargetInfos")
		UE_API FTargetInfo GetBuildTargetInfo(const ILauncherProfileRef& Profile);
		UE_API TArray<FTargetInfo> GetBuildTargetInfos(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun);

		/* Get a device proxy for a profile */
		static UE_API TSharedPtr<ITargetDeviceProxy> GetDeviceProxy(const ILauncherProfileBuildCookRunRef& BuildCookRun);

		/* Update device proxies based on the current cook platforms */
		UE_API void UpdateDeployDeviceProxyFromCookedPlatforms(const ILauncherProfileRef& Profile);

		/* Update cooked platforms based on selected device proxy */
		UE_API void UpdatedCookedPlatformsFromDeployDeviceProxy(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun, TSharedPtr<ITargetDeviceProxy> DeviceProxy = nullptr);

		/* Update cooked platforms based on build target */
		UE_API void UpdateCookedPlatformsFromBuildTarget(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun);

		/* Attempts to set a valid build target if one has not been specified */
		UE_API void EnsureValidBuildTarget(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun);

		/* Utility functions for resolving platform names */
		static UE_API FString GetVanillaPlatformName(const FString& PlatformName);
		static UE_API FString GetPlatformNameWithFlavor(const FString& PlatformName);
		static UE_API FString GetBuildTargetPlatformName(const FString& PlatformName, const FTargetInfo& BuildTargetInfo);

		/* Retrieve project settings from a project path or profile */
		UE_API const FProjectSettings GetProjectSettings(const FString& ProjectPath, bool bWait = true);
		UE_API const FProjectSettings GetProjectSettings(const ILauncherProfileRef& Profile, bool bWait = true);
		UE_API bool AreProjectSettingsReady(const FString& ProjectPath);
		UE_API bool AreProjectSettingsReady(const ILauncherProfileRef& Profile);
		UE_API FString GetProjectPath(const ILauncherProfileRef& Profile) const;

		/* Delegate fired when a project's settings have been loaded */
		FOnProjectSettingsReady& OnProjectSettingsReady()
		{
			return ProjectSettingsReady;
		}


		/* Get config section and ini file used for storing project launcher state */
		UE_API const TCHAR* GetConfigSection() const;
		UE_API const FString& GetConfigIni() const;

		/* Returns true if extensions are enabled for project launcher */
		UE_API bool AreExtensionsEnabled() const;

		/* Returns true if we can use a simplified layout for this profile */
		UE_API bool CanUseSimplifiedLayout(const ILauncherProfileRef& Profile);

		/* Get available maps for a project, engine, or plugin */
		UE_API TArray<FString> GetAvailableProjectMapNames(const FString& InProjectPath, bool bIncludeNonContentDirMaps = false);
		UE_API TArray<FString> GetAvailableProjectMapPaths(const FString& InProjectPath, bool bIncludeNonContentDirMaps = false);
		UE_API TArray<FString> GetAvailableEngineMapNames();
		UE_API TArray<FString> GetAvailableEngineMapPaths();
		UE_API TArray<FString> GetAvailablePluginMapNames(const FString& PluginContentDir);
		UE_API TArray<FString> GetAvailablePluginMapPaths(const FString& PluginContentDir);

		/* Get the current UAT command information for the given profile */
		struct FUATCommandsDetail
		{
			int32 MinOrder = 0;
			int32 MaxOrder = 0;
			uint32 NumUATCommands = 0;
		};
		UE_API void GetUATCommandsDetail( const ILauncherProfileRef& Profile, FUATCommandsDetail& OutUATCommandsDetail);

		/* Set the defaults for this BuildCookRun */
		void SetDefaults(const ILauncherProfileRef& Profile, const ILauncherProfileBuildCookRunRef& BuildCookRun);

	private:
		/* Load and save config state */
		UE_API void LoadConfig();
		UE_API void SaveConfig();

		/* Handlers for profile manager and device proxy events */
		UE_API void HandleProfileManagerProfileAdded(const ILauncherProfileRef& Profile);
		UE_API void HandleProfileManagerProfileRemoved(const ILauncherProfileRef& Profile);
		UE_API void HandleProfileManagerFallbackProjectChanged();
		UE_API void HandleDeviceProxyAdded(const TSharedRef<ITargetDeviceProxy>& DeviceProxy);
		UE_API void HandleDeviceProxyRemoved(const TSharedRef<ITargetDeviceProxy>& DeviceProxy);

		/* Handler for profile properties changing */
		UE_API void HandleProfileProjectChanged( ILauncherProfilePtr InProfile );
		UE_API void HandleProjectSettingsReady( const FString& ProjectPath );

		/* Helper for reading project settings from config */
		UE_API void ReadProjectSettingsFromConfig(FConfigCacheIni& InConfig, const FString& InProjectPath, FProjectSettings& OutResult);

		/* Modify launch command lines to inject extension variables */
		UE_API void OnModifyLaunchCommandLine( const ILauncherProfileRef& InProfile, EBuildTargetType InBuildTargetType, FString& InOutCommandLine );
		UE_API void OnModifyUATCommandLine( FUATCommandPostProcessParameters& PostProcessParameters );

		/* Get map paths under a content directory, caching results */
		UE_API TArray<FString> GetAndCacheMapPaths(const FString& InContentDir, bool bIncludeNonContentDirMaps = false);

	private:
		FString ConfigFileName;
		
		TSharedRef<ITargetDeviceProxyManager> DeviceProxyManager;
		TSharedRef<ILauncher> Launcher;
		TSharedRef<ILauncherProfileManager> ProfileManager;

		ILauncherProfilePtr SelectedProfile;
		TArray<ILauncherProfilePtr> AllProfiles;
		ILauncherProfilePtr BasicLaunchProfile;
		bool bHasSetBasicLaunchProfilePlatform = false;

		friend class FLaunchLogTextLayoutMarshaller;
		TArray<TSharedPtr<FLaunchLogMessage>> LaunchLogMessages;

		FCriticalSection CachedProjectSettingsCS;
		TMap<FString,FProjectSettings> CachedProjectSettings;
		TMap<FString,TArray<FString>> CachedMapPaths;

		FOnSelectedProfileChanged ProfileSelectedDelegate;
		FOnSelectedPlatformChanged PlatformChangedDelegate;
		FOnProjectSettingsReady ProjectSettingsReady;

		ILauncherProfilePtr DefaultBasicLaunchProfile;
		ILauncherProfilePtr DefaultCustomLaunchProfile;
	};
}

#undef UE_API
