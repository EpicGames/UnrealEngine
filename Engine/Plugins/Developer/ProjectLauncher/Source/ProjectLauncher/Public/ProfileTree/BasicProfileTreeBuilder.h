// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfileTree/GenericProfileTreeBuilder.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{

	class FBasicProfileTreeBuilder : public FGenericProfileTreeBuilder
	{
	public:
		FBasicProfileTreeBuilder( const ILauncherProfileRef& Profile, const TSharedRef<FModel>& InModel );
		virtual ~FBasicProfileTreeBuilder() = default;

		virtual void Construct() override;

		virtual FString GetName() const override
		{
			return TEXT("BasicProfile");
		}
	};




	class FBasicProfileTreeBuilderFactory : public ILaunchProfileTreeBuilderFactory
	{
	public:
		UE_API virtual TSharedPtr<ILaunchProfileTreeBuilder> TryCreateTreeBuilder( const ILauncherProfileRef& InProfile, const TSharedRef<FModel>& InModel ) override;

		virtual bool IsProfileTypeSupported(EProfileType ProfileType ) const override { return ProfileType == EProfileType::Basic; }

	};
}

#undef UE_API
