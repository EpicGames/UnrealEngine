// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfileTree/GenericProfileTreeBuilder.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	class FCustomProfileTreeBuilder : public FGenericProfileTreeBuilder
	{
	public:
		UE_API FCustomProfileTreeBuilder( const ILauncherProfileRef& Profile, const TSharedRef<FModel>& InModel );
		virtual ~FCustomProfileTreeBuilder() = default;

		UE_API virtual void Construct() override;

		virtual FString GetName() const override
		{
			return TEXT("CustomProfile");
		}
	};


	class FCustomProfileTreeBuilderFactory : public ILaunchProfileTreeBuilderFactory
	{
	public:
		UE_API virtual TSharedPtr<ILaunchProfileTreeBuilder> TryCreateTreeBuilder( const ILauncherProfileRef& Profile, const TSharedRef<FModel>& InModel ) override;

		virtual bool IsProfileTypeSupported(EProfileType ProfileType ) const override { return ProfileType == EProfileType::Custom; }

	};

}

#undef UE_API
