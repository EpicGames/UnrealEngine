// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfileTree/ILaunchProfileTreeBuilder.h"
#include "Model/ProjectLauncherModel.h"
#include "ILauncherProfileBuildCookRun.h"
#include "Containers/Map.h"
#include "Containers/Set.h"

#define UE_API PROJECTLAUNCHER_API

class SSearchableComboBox;
class SCustomLaunchPlatformCombo;
class SCustomLaunchBuildTargetCombo;
class SCustomLaunchDeviceListView;
namespace ESelectInfo { enum Type : int; }

namespace ProjectLauncher
{
	/**
	 * Base class for a profile tree builder that creates FLaunchProfileTreeData from a given ILauncherProfile.
	 * 
	 * Expected to be created by an instance of ILaunchProfileTreeBuilderFactory, for example:
	 * 
	 *    TSharedPtr<ILaunchProfileTreeBuilder> FMyProfileTreeBuilderFactory::TryCreateTreeBuilder( const ILauncherProfileRef& InProfile, const TSharedRef<FModel>& InModel )
	 *    {
	 *        return MakeShared<FMyProfileTreeBuilder>(InProfile, InModel);
	 *    }
	 */
	class FGenericProfileTreeBuilder : public ILaunchProfileTreeBuilder, public TSharedFromThis<FGenericProfileTreeBuilder>
	{
	public:
		UE_API FGenericProfileTreeBuilder( const ILauncherProfileRef& Profile, const ILauncherProfileRef& InDefaultProfile, const TSharedRef<FModel>& InModel );
		virtual ~FGenericProfileTreeBuilder();

		UE_API virtual void Construct() override;

		virtual FLaunchProfileTreeDataRef GetProfileTree() override
		{
			return TreeData;
		}

		virtual bool AllowExtensionsUI() const override
		{
			return true;
		}

	protected:
		// default property creation functions
		UE_API void AddProjectProperty( FLaunchProfileTreeNode& HeadingNode );

		UE_API FString GetProjectPath() const;
		UE_API void SetProjectName(FString ProjectPath);
		UE_API bool HasProject() const;
		UE_API void ValidateBuildTargets(const FString& ProjectPath);

		class FBuildCookRun : public TSharedFromThis<FBuildCookRun>
		{
		public:
			FBuildCookRun( FGenericProfileTreeBuilder& InOwner, ILauncherProfileBuildCookRunRef InBuildCookRun, const TSharedRef<FModel>& InModel );
			~FBuildCookRun();

			UE_API void AddTargetProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddPlatformProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddConfigurationProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddContentSchemeProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddCompressPakFilesProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddGenerateChunksProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddZenSnapshotProperty(FLaunchProfileTreeNode& HeadingNode);
			UE_API void AddImportZenSnapshotProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddZenPakStreamingPathProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddSubmissionPackageProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddIncrementalCookProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddCookProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddBuildProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddForceBuildProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddArchitectureProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddStagingDirectoryProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddArchiveBuildProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddArchiveBuildDirectoryProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddDeployProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddIncrementalDeployProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddTargetDeviceProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddRunProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddInitialMapProperty( FLaunchProfileTreeNode& HeadingNode );
			UE_API void AddCommandLineProperty( FLaunchProfileTreeNode& HeadingNode );


			enum class EDeployDeviceOption : uint8
			{
				Default,
				Selected,
			};

			UE_API void Construct();


			UE_API TArray<FString> GetBuildTargets() const;
			UE_API void SetBuildTargets( TArray<FString> BuildTargets);

			UE_API void SetBuildConfiguration(EBuildConfiguration BuildConfiguration);
			UE_API EBuildConfiguration GetBuildConfiguration() const;

			UE_API void SetContentScheme(EContentScheme ContentScheme);
			UE_API bool IsContentSchemeAvailable(EContentScheme, FText& OutReason) const;

			UE_API FString GetCommandLine(EBuildTargetType BuildTarget) const;
			UE_API void SetCommandLine( const FString& NewCommandLine, EBuildTargetType BuildTarget );

			UE_API void SetSelectedPlatforms( TArray<FString> SelectedPlatforms );
			UE_API TArray<FString> GetSelectedPlatforms() const;

			UE_API void SetCook( bool bCook );
			UE_API bool GetCook( ILauncherProfileBuildCookRunPtr InBuildCookRun = nullptr ) const;

			UE_API void SetDeployDeviceIDs(TArray<FString> DeployDeviceIDs);
			UE_API TArray<FString> GetDeployDeviceIDs() const;
			UE_API EDeployDeviceOption GetDeployDeviceOption() const;
			UE_API void SetDeployDeviceOption( EDeployDeviceOption DeployDeviceOption );
			UE_API float GetDeployDeviceListHeight() const;
			UE_API void SetDeployDeviceListHeight( float NewHeight );
			UE_API void OnDeviceRemoved(FString DeviceID);
			UE_API TSharedRef<SWidget> CreateDeployDeviceWidget();

			UE_API void SetBuild( bool bBuild );
			UE_API bool GetBuild( ILauncherProfileBuildCookRunPtr InBuildCookRun = nullptr ) const;

			UE_API void SetForceBuild( bool bForceBuild );
			UE_API bool GetForceBuild( ILauncherProfileBuildCookRunPtr InBuildCookRun = nullptr ) const;

			UE_API void SetArchitecture( FString Architecture, EBuildTargetType TargetType );
			UE_API FString GetArchitecture(EBuildTargetType TargetType) const;
			UE_API FText GetArchitectureDisplayName( FString Architecture );

			UE_API void SetDeployToDevice( bool bDeployToDevice );
			UE_API bool GetDeployToDevice( ILauncherProfileBuildCookRunPtr InBuildCookRun = nullptr ) const;

			UE_API void SetIsRunning( bool bRun );
			UE_API bool GetIsRunning( ILauncherProfileBuildCookRunPtr InBuildCookRun = nullptr ) const;

			UE_API void OnInitialMapChanged( TSharedPtr<FString> InitialMap, ESelectInfo::Type );
			UE_API TSharedPtr<FString> GetInitialMap() const;


			UE_API void SetHasAdvancedPlatformTargets( bool bEnable );
			UE_API bool HasAdvancedPlatformTargets() const;

			UE_API void RefreshContentScheme();
			UE_API void ValidateArchitectures();
			UE_API void OnValidateProfile(ILauncherProfileUATCommandRef UATCommand);
			UE_API void OnProjectSettingsReady(const FString& ProjectPath);

			UE_API void CacheArchitectures();
			UE_API void CacheBuildTargets();

			FText GetZenSnapshotText() const;

			// helper callbacks to simplify control enable/visibility
			FLaunchProfileTreeNode::FGetBool ForPak;
			FLaunchProfileTreeNode::FGetBool ForZen;
			FLaunchProfileTreeNode::FGetBool ForZenWS;
			FLaunchProfileTreeNode::FGetBool ForCooked;
			FLaunchProfileTreeNode::FGetBool ForEnabledCooked;
			FLaunchProfileTreeNode::FGetBool ForContent;
			FLaunchProfileTreeNode::FGetBool ForCode;
			FLaunchProfileTreeNode::FGetBool ForCodeBuild;
			FLaunchProfileTreeNode::FGetBool ForDeployment;
			FLaunchProfileTreeNode::FGetBool ForRun;
			FLaunchProfileTreeNode::FGetBool ForSubmissionPackage;
			FLaunchProfileTreeNode::FGetString EmptyString;

			TArray<FString> CachedDeployDeviceIDs;

			float DeployDeviceListHeight = 150.0f;

			EContentScheme ContentScheme;
			bool bHasAdvancedPlatformTargets;
			bool bShouldCook;
			TSharedPtr<SCustomLaunchPlatformCombo> PlatformCombo;
			TSharedPtr<SCustomLaunchBuildTargetCombo> BuildTargetCombo;
			TSharedPtr<SSearchableComboBox> InitalMapCombo;
			TSharedPtr<SCustomLaunchDeviceListView> DeployDeviceListView;
			TMap<EBuildTargetType, TArray<FString>> CachedBuildTargetArchitectures;
			TSet<EBuildTargetType> CachedBuildTargetTypes;

			FGenericProfileTreeBuilder& Owner;
			ILauncherProfileBuildCookRunRef BuildCookRun;
			const TSharedRef<FModel> Model;
		};


	protected:

		void OnUATCommandAdded(const ILauncherProfileUATCommandRef& UATCommand);
		void OnUATCommandRemoved(const ILauncherProfileUATCommandRef& UATCommand);
		UE_API virtual void OnPropertyChanged() override;
		UE_API void CacheStartupMapList() const;
		UE_API TSharedRef<SWidget> OnGenerateComboWidget( TSharedPtr<FString> InComboString );


		mutable bool bStartupMapCacheDirty = true;
		mutable TArray<TSharedPtr<FString>> CachedStartupMaps;

		FLaunchProfileTreeDataRef TreeData;
		const ILauncherProfileRef Profile;
		const ILauncherProfileRef DefaultProfile;
		ILauncherProfileBuildCookRunPtr DefaultBuildCookRun;
		EProfileType ProfileType;

		const TSharedRef<FModel> Model;

		FBuildCookRun& Get(const ILauncherProfileBuildCookRunRef& BuildCookRun);




		TMap<ILauncherProfileBuildCookRunRef,TSharedRef<FBuildCookRun>> BuildCookRunData;
	};
}

#undef UE_API
