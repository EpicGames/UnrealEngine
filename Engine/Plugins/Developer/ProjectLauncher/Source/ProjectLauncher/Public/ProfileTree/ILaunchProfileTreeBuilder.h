// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ProfileTree/LaunchProfileTreeData.h"
#include "Model/ProjectLauncherModel.h"
#include "ILauncherProfile.h"

namespace ProjectLauncher
{

	/**
	 * Interface for a profile tree builder that creates FLaunchProfileTreeData from a given ILauncherProfile.
	 * 
	 * Expected to be created by an instance of ILaunchProfileTreeBuilderFactory, for example:
	 * 
	 *    TSharedPtr<ILaunchProfileTreeBuilder> FMyProfileTreeBuilderFactory::TryCreateTreeBuilder( const ILauncherProfileRef& InProfile, const TSharedRef<FModel>& InModel )
	 *    {
	 *        return MakeShared<FMyProfileTreeBuilder>(InProfile, InModel);
	 *    }
	 */
	class ILaunchProfileTreeBuilder
	{
	public:
		/**
		 * Construct the launch profile tree
		 */
		virtual void Construct() = 0;

		/**
		 * Provide access to the launch profile tree
		 * 
		 * @returns profile UI tree
		 */
		virtual FLaunchProfileTreeDataRef GetProfileTree() = 0;

		/**
		 * Debug name for this tree builder
		 * 
		 * @returns debug name of the tree builder
		 */
		virtual FString GetName() const = 0;

		/**
		 * Callback when the property tree editor has modified the profile.
		 * NOTE: Custom widgets will need to call this manually.
		 */
		virtual void OnPropertyChanged() = 0;

		/**
		 * Whether this tree builder allows extensions to add UI elements. Typically would be true without good reason.
		 */
		virtual bool AllowExtensionsUI() const = 0;
	};


	/**
	 * Interface for a factory that can create specializations of ILaunchProfileTreeBuilder for a given ILauncherProfile
	 * 
	 * Singleton instance is registered with this plugin during initialization as follows:
	 * 
	 *   TSharedPtr<FMyTreeBuilderFactory> MyTreeBuilder = MakeShared<FMyTreeBuilderFactory>();
	 *   IProjectLauncherModule::Get().RegisterTreeBuilder( MyTreeBuilder.ToSharedRef() );
	 */
	class ILaunchProfileTreeBuilderFactory
	{
	public:
		/**
		 * Create a profile tree builder class for the given profile
		 * 
		 * @param Profile the profile being edited
		 * @param Model helper class
		 * @returns new instance of a profile tree builder for the given profile, or null if it is not supported by this tree builder factory
		 */
		virtual TSharedPtr<ILaunchProfileTreeBuilder> TryCreateTreeBuilder( const ILauncherProfileRef& Profile, const TSharedRef<ProjectLauncher::FModel>& Model ) = 0;

		/**
		 * Get the priority for this tree builder factory. Higher priority tree builders are evaluated first. 
		 * Specializations should return non-zero values
		 * 
		 * @returns priority for this tree builder factory
		 */
		virtual int GetPriority() const { return 0; }

		/**
		 * Determines whether this tree builder can support profiles of the given type
		 * 
		 * @param ProfileType the type of profile
		 * @returns true if the profile type is supported
		 */
		virtual bool IsProfileTypeSupported(ProjectLauncher::EProfileType ProfileType ) const = 0;
	};
}


