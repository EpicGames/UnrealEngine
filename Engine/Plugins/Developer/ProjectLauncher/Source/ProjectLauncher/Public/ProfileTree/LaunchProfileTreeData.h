// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Math/Interval.h"
#include "Widgets/Input/SSpinBox.h"
#include "ILauncherProfile.h"
#include <initializer_list>

#define UE_API PROJECTLAUNCHER_API


class SWidget;
struct EVisibility;

namespace ProjectLauncher
{
	class ILaunchProfileTreeBuilder;
	class FLaunchProfileTreeData;
	class FLaunchProfileTreeNode;
	class FLaunchExtensionInstance;
	class FModel;

	typedef TSharedPtr<FLaunchProfileTreeData> FLaunchProfileTreeDataPtr;
	typedef TSharedRef<FLaunchProfileTreeData> FLaunchProfileTreeDataRef;

	typedef TSharedPtr<FLaunchProfileTreeNode> FLaunchProfileTreeNodePtr;
	typedef TSharedRef<FLaunchProfileTreeNode> FLaunchProfileTreeNodeRef;

	/*
	 * Validation object representing errors and warnings tied to profile settings. 
	 * If the given validation errors are present in the profile a warning icon will be displayed next to this property in the UI
	 *
	 * Use with the Validation callback in FLaunchProfileTreeNode:
	 *
	 *  .Validation = FValidation( {ELauncherProfileValidationErrors::NoProjectSelected, ...} ),
	 *     or
	 *  .Validation = FValidation( {TEXT("Validation_SomeCustomValidation"), ...} ),
	 * 
	 */
	class FValidation
	{
	public:
		FValidation() {}
		UE_API FValidation( std::initializer_list<ELauncherProfileValidationErrors::Type> InAssociatedErrors );
		UE_API FValidation( std::initializer_list<ELauncherProfileValidationErrors::Type> InAssociatedErrors, std::initializer_list<FString> InAssociatedCustomErrors );
		UE_API FValidation( std::initializer_list<FString> InAssociatedCustomErrors );

		/* Returns true if any validation rules are set. */
		bool IsSet() const;

		/* Returns true if the given profile has one of the associated errors. */
		bool HasError( ILauncherProfileRef Profile, ILauncherProfileUATCommandPtr UATCommand ) const;

		/* Collects error text from the profile for display. */
		void GetErrorText( ILauncherProfileRef Profile, ILauncherProfileUATCommandPtr UATCommand, TArray<FString>& ErrorLines ) const;

	private:
		bool bIsSet = false;                                             // whether this validation has been configured
		TArray<ELauncherProfileValidationErrors::Type> AssociatedErrors; // engine-defined errors
		TArray<FString> AssociatedCustomErrors;                          // custom errors specific to this profile
	};

	/*
	 * Represents a single editable node (field, checkbox, widget, etc.) in the profile tree UI.
	 */
	class FLaunchProfileTreeNode : public TSharedFromThis<FLaunchProfileTreeNode>
	{
	public:
		/* Construct a node bound to a tree. */
		UE_API FLaunchProfileTreeNode(FLaunchProfileTreeData* InTreeData);

		typedef TFunction<void()> FFunction;
		typedef TFunction<bool()> FGetBool;
		typedef TFunction<void(bool)> FSetBool;
		typedef TFunction<int32()> FGetInt;
		typedef TFunction<void(int32)> FSetInt;
		typedef TFunction<float()> FGetFloat;
		typedef TFunction<void(float)> FSetFloat;
		typedef TFunction<FString(void)> FGetString;
		typedef TFunction<void(FString)> FSetString;

		/* Callbacks common to all widgets. */
		struct FCallbacks
		{
			FGetBool IsDefault = nullptr;     // check if current value is default
			FFunction SetToDefault = nullptr; // reset to default
			FGetBool IsVisible = nullptr;     // visibility logic
			FGetBool IsEnabled = nullptr;     // enabled/disabled logic
			FValidation Validation;           // validation rules
		};

		/* Add a generic widget with callbacks. */
		UE_API FLaunchProfileTreeNode& AddWidget( FText InName, FCallbacks&& InWidgetCallbacks, TSharedRef<SWidget> InValueWidget );

		/* Add a generic widget without callbacks. */
		UE_API FLaunchProfileTreeNode& AddWidget( FText InName, TSharedRef<SWidget> InValueWidget );

		/* Boolean-specific callbacks. */
		struct FBooleanCallbacks
		{
			FGetBool GetValue = nullptr;
			FSetBool SetValue = nullptr;
			FGetBool GetDefaultValue = nullptr;
			FGetBool IsVisible = nullptr;
			FGetBool IsEnabled = nullptr;
			FValidation Validation;
		};
		UE_API FLaunchProfileTreeNode& AddBoolean( FText InName, FBooleanCallbacks&& BooleanCallbacks, FText ToolTipText = FText::FromString(""));

		/* String-specific callbacks. */
		struct FStringCallbacks
		{
			FGetString GetValue = nullptr;
			FSetString SetValue = nullptr;
			FGetString GetDefaultValue = nullptr;
			FGetBool IsVisible = nullptr;
			FGetBool IsEnabled = nullptr;
			FValidation Validation;
		};
		UE_API FLaunchProfileTreeNode& AddString( FText InName, FStringCallbacks&& StringCallbacks, FText ToolTipText = FText::FromString(""), FText HintText = FText::FromString(""));
		UE_API FLaunchProfileTreeNode& AddDirectoryString(FText InName, FStringCallbacks&& StringCallbacks, bool InMakeRelative = true, FText ToolTipText = FText::FromString(""));
		UE_API FLaunchProfileTreeNode& AddFileString(FText InName, FStringCallbacks&& StringCallbacks, FString FileFilter = TEXT("All files (*.*)|*.*"));
		UE_API FLaunchProfileTreeNode& AddCommandLineString( FText InName, FStringCallbacks&& StringCallbacks, const TCHAR* InType = nullptr ); // defaults to "Launch" cmdline extensions

		/* Generic value callbacks. */
		template<typename T>
		struct TTypeCallbacks
		{
			TFunction<T()> GetValue = nullptr;
			TFunction<void(T)> SetValue = nullptr;
			TFunction<T()> GetDefaultValue = nullptr;
			FGetBool IsVisible = nullptr;
			FGetBool IsEnabled = nullptr;
			FValidation Validation;
		};

		/* Add a typed value widget, e.g. spinner for int or float. */
		template<typename T>
		FLaunchProfileTreeNode& AddValue( FText InName, TTypeCallbacks<T>&& TypeCallbacks, TInterval<T> Range = TInterval<T>(), bool bShowSpinner = false );

		UE_API FLaunchProfileTreeNode& AddInteger( FText InName, TTypeCallbacks<int32>&& TypeCallbacks, FInt32Interval Range = FInt32Interval(), bool bShowSpinner = false );
		UE_API FLaunchProfileTreeNode& AddFloat( FText InName, TTypeCallbacks<float>&& TypeCallbacks, FFloatInterval Range = FFloatInterval(), bool bShowSpinner = false );

		UE_API FLaunchProfileTreeNode& AddSubHeading(const TCHAR* InName, FText InDisplayName, int32 SortOrder = 0);
		UE_API FLaunchProfileTreeNode& GetSubHeading(const TCHAR* InName) const;

		UE_API FLaunchProfileTreeNode& FindOrAddSubHeading(ILauncherProfileUATCommandRef UATCommand);

		UE_API bool ContainsNodeRecursive( const FLaunchProfileTreeNodePtr& TargetNode ) const;

		/* Node details */
		FText Name;													// label for the node
		int32 SortOrder;											// sort order for headings
		int32 Depth = 0;											// tree depth
		TSharedPtr<SWidget> Widget;									// UI widget
		ILauncherProfileUATCommandPtr UATCommand;					// option UAT command associated with this node
		FCallbacks Callbacks;										// generic callbacks
		TArray<FLaunchProfileTreeNodePtr> Children;					// child nodes
		TMap<FString,FLaunchProfileTreeNodePtr> SubHeadingNodes;    // named subheadings

		const FLaunchProfileTreeData* GetTreeData() const { return TreeData; }
		FLaunchProfileTreeData* GetTreeData() { return TreeData; }

	protected:
		/* Notify parent tree that a property has changed. */
		UE_API void OnPropertyChanged();

		FLaunchProfileTreeData* TreeData;
	};

	/*
	 * Represents the full profile tree of editable UI nodes for a launcher profile.
	 * Integrates extensions and manages node headings.
	 */
	class FLaunchProfileTreeData : public TSharedFromThis<FLaunchProfileTreeData>
	{
	public:
		/* Construct with a profile, model, and tree builder. */
		UE_API FLaunchProfileTreeData(ILauncherProfilePtr InProfile, TSharedRef<FModel> InModel, ILaunchProfileTreeBuilder* InTreeBuilder);
		UE_API ~FLaunchProfileTreeData();

		/* Add or retrieve top-level headings. */
		UE_API FLaunchProfileTreeNode& AddHeading(const TCHAR* InName, FText InDisplayName, int32 SortOrder = 0);
		UE_API FLaunchProfileTreeNode& GetHeading(const TCHAR* InName) const;

		UE_API FLaunchProfileTreeNode& FindOrAddHeading(ILauncherProfileUATCommandRef UATCommand, bool bIsPrimaryHeading);
		UE_API void SortNodes();

		/* Invoke extension instances to build their UI. */
		UE_API void CreateExtensionsUI();

		/* Notify of property changes and refresh requests. */
		UE_API void OnPropertyChanged();
		UE_API void RequestTreeRefresh();
		UE_API void RequestFullTreeRebuild();
		UE_API void OnCommandLineFieldCreated( FLaunchProfileTreeNode& InNode, const FLaunchProfileTreeNode::FStringCallbacks& InStringCallbacks, const FString& InTypeName );
		UE_API void OnLaunchExtensionToggle( const TSharedRef<class FLaunchExtensionInstance>& ExtensionInstance, bool bAdded );

		/* Member variables */
		ILauncherProfilePtr Profile;                                     // associated profile
		TSharedRef<FModel> Model;                                        // owning model
		TArray<FLaunchProfileTreeNodePtr> Nodes;                         // all nodes
		TMap<FString,FLaunchProfileTreeNodePtr> HeadingNodes;            // named top-level headings
		ILaunchProfileTreeBuilder* TreeBuilder;                          // UI builder interface
		TArray<TSharedPtr<FLaunchExtensionInstance>> ExtensionInstances; // extension instances attached
		TSet<FLaunchProfileTreeNodePtr> RequestExpandHeadings;           // headings to expand
		TSet<FLaunchProfileTreeNodePtr> UATCommandHeadings;              // headings that have additional controls for modifying the command
		bool bRequestTreeRefresh = false;                                // request to refresh tree UI
		bool bRequestTreeFullRebuild = false;                            // request to fully recreate the profile tree

		/* Constants */
		UE_API static const TCHAR* GeneralSettingsSectionName;
		UE_API static const int32 GeneralSettingsSortOrder;
		UE_API static const int32 DefaultSortOrder;

	private:
		struct FCommandLineField
		{
			FLaunchProfileTreeNode::FGetString GetValue = nullptr;
			FLaunchProfileTreeNode::FSetString SetValue = nullptr;
			FString TypeName;
		};
		TArray<FCommandLineField> AllCommandLineFields;
	};




	template<typename T>
	FLaunchProfileTreeNode& FLaunchProfileTreeNode::AddValue( FText InName, TTypeCallbacks<T>&& TypeCallbacks, TInterval<T> Range, bool bShowSpinner )
	{
		check(TypeCallbacks.GetValue);
		check(TypeCallbacks.SetValue);

		FCallbacks WidgetCallbacks
		{
			.IsVisible = TypeCallbacks.IsVisible,
			.Validation = TypeCallbacks.Validation,
		};
		if (TypeCallbacks.GetDefaultValue != nullptr)
		{
			WidgetCallbacks.IsDefault = [TypeCallbacks]()
			{
				return TypeCallbacks.GetValue() == TypeCallbacks.GetDefaultValue();
			};

			WidgetCallbacks.SetToDefault = [TypeCallbacks]()
			{
				TypeCallbacks.SetValue(TypeCallbacks.GetDefaultValue());
				// nb. OnPropertyChanged is called via SCustomLaunchWidgetTableRow's set to default
			};
		};

		auto SetImmediateValue = [this, Range, TypeCallbacks]( T Value )
		{
			if (Range.IsValid())
			{
				TypeCallbacks.SetValue( FMath::Clamp( Value, Range.Min, Range.Max) );
			}
			else
			{
				TypeCallbacks.SetValue( Value );
			}
		};

		auto SetValue = [this, SetImmediateValue]( T Value, ETextCommit::Type )
		{
			SetImmediateValue(Value);
			OnPropertyChanged();
		};

		auto GetValue = [TypeCallbacks]()
		{
			return TypeCallbacks.GetValue();
		};

		auto IsEnabled = [TypeCallbacks]()
		{
			return !TypeCallbacks.IsEnabled.IsSet() || TypeCallbacks.IsEnabled();
		};

		if (Range.IsValid())
		{
			return AddWidget( InName,
				MoveTemp(WidgetCallbacks),
				SNew(SSpinBox<T>)
				.Value_Lambda(GetValue)
				.OnValueChanged_Lambda(SetImmediateValue)
				.OnValueCommitted_Lambda(SetValue)
				.MinValue(Range.Min)
				.MaxValue(Range.Max)
				.EnableSlider(bShowSpinner)
				.IsEnabled_Lambda(IsEnabled)
			);
		}
		else
		{
			return AddWidget( InName,
				MoveTemp(WidgetCallbacks),
				SNew(SSpinBox<T>)
				.Value_Lambda(GetValue)
				.OnValueCommitted_Lambda(SetValue)
				.EnableSlider(false)
				.IsEnabled_Lambda(IsEnabled)
			);
		}
	}
}

#undef UE_API
