// Copyright Epic Games, Inc. All Rights Reserved.


#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

namespace ProjectLauncher
{
	class ILaunchProfileTreeBuilderFactory;
	class FLaunchExtension;
};

/*
 * Interface for ProjectLauncher module
 */
class IProjectLauncherModule : public IModuleInterface
{
public:
	/**
	* Singleton-like access to the Custom Launch UI module instance
	* @return Returns IProjectLauncherModule singleton instance, loading the module on demand if needed
	*/
	static inline IProjectLauncherModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IProjectLauncherModule>("ProjectLauncher");
	}

	/**
	* Singleton-like access to the Custom Launch UI module instance
	* @return Returns IProjectLauncherModule singleton instance if it is loaded
	*/
	static inline IProjectLauncherModule* TryGet()
	{
		return FModuleManager::GetModulePtr<IProjectLauncherModule>("ProjectLauncher");
	}



	/**
	 * Registers a marhaller factory, for defining the layout & fields for editing an ILauncherProfile
	 */
	virtual void RegisterTreeBuilder( TSharedRef<ProjectLauncher::ILaunchProfileTreeBuilderFactory> TreeBuilderFactory ) = 0;

	/**
	 * Unregisters a previously-registered tree builder factory
	 */
	virtual void UnregisterTreeBuilder( TSharedRef<ProjectLauncher::ILaunchProfileTreeBuilderFactory> TreeBuilderFactory ) = 0;



	/**
	 * Registers an extension
	 */
	virtual void RegisterExtension( TSharedRef<ProjectLauncher::FLaunchExtension> Extension ) = 0;

	/**
	 * Unregisters a previously-registered extension
	 */
	virtual void UnregisterExtension( TSharedRef<ProjectLauncher::FLaunchExtension> Extension ) = 0;
};

