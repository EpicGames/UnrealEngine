// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/UnrealString.h"

#define UE_API PROJECTLAUNCHER_API

namespace ProjectLauncher
{
	namespace CmdLineUtils
	{
		/**
		 * Determine if the given parameter is on the command line
 		 * 
		 * @param InParameter the parameter to test, e.g. -key=value or -param
		 * @returns whether the parameter is being used
		 */
		UE_API bool IsParameterUsed( const FString& InCmdLine, const FString& InParameter );

		/** 
		 * Add or remove the given parameter on the command line
		 * 
		 * @param InParameter the parameter to add or remove, e.g. -key=value or -param
		 * @param bUsed whether to add or remove the parameter
		 */
		UE_API void SetParameterUsed( FString& InOutCmdLine, const FString& InParameter, bool bUsed );

		/** 
		 * Add the given parameter to the command line
		 * 
		 * @param InParameter the parameter to add, e.g. -key=value or -param
		 */
		UE_API void AddParameter( FString& InOutCmdLine, const FString& InParameter );

		/** 
		 * Remove the given parameter from the command line
		 * 
		 * @param InParameter the parameter to remove, e.g. -key=value or -param
		 * @returns true if the parameter was removed
		 */
		UE_API bool RemoveParameter( FString& InOutCmdLine, const FString& InParameter );

		/** 
		 * Retrieve the current value for the given command line parameter
		 * 
		 * @param InParameter the parameter to query, e.g. -key=value
		 * @returns the value of the parameter
		 */
		UE_API FString GetParameterValue( const FString& InCmdLine, const FString& InParameter );

		/** 
		 * Update the value of the given command line parameter
		 * 
		 * @param InParameter the parameter to modify, e.g. -key=value or -key=
		 * @param InNewValue the new value to use
		 * @returns true unless InParameter is not a key/value property
		 */
		UE_API bool UpdateParameterValue( FString& InOutCmdLine, const FString& InParameter, const FString& InNewValue );

		/** 
		 * Get the current state of the given command line parameter, allowing for user-changed values
		 * 
		 * @param InParameter the parameter to test, e.g. -key=value
		 * @returns The parameter as it is now, allowing for user modifications, e.g. -key=value or -key=some_new_value
		 */
		UE_API FString GetFinalParameter( const FString& InCmdLine, const FString& InParameter );
	};
};

#undef UE_API
