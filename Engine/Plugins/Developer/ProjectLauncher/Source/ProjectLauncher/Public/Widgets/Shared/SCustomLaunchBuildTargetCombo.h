// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Framework/SlateDelegates.h"
#include "Fonts/SlateFontInfo.h"
#include "Model/ProjectLauncherModel.h"

#define UE_API PROJECTLAUNCHER_API

class SCustomLaunchBuildTargetCombo
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TArray<FString> );

	SLATE_BEGIN_ARGS(SCustomLaunchBuildTargetCombo)
		: _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		{}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_ATTRIBUTE(FString, SelectedProject);
		SLATE_ATTRIBUTE(TArray<FString>, SelectedBuildTargets);
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_ATTRIBUTE(bool, GetAdvancedPlatformsOption)
		SLATE_EVENT(FOnBooleanValueChanged, SetAdvancedPlatformsOption)
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )
	SLATE_END_ARGS()

public:
	UE_API void Construct(	const FArguments& InArgs, TSharedRef<ProjectLauncher::FModel> InModel);

protected:
	UE_API FString GetDefaultBuildTargetName() const;
	UE_API ECheckBoxState GetBuildTargetCheckState( FString BuildTargetName ) const;

	TAttribute<FString> SelectedProject;
	TAttribute<TArray<FString>> SelectedBuildTargets;
	FOnSelectionChanged OnSelectionChanged;
	bool bShowAnyProjectOption;
	TAttribute<bool> GetAdvancedPlatformsOption;
	FOnBooleanValueChanged SetAdvancedPlatformsOption;

	TSharedPtr<ProjectLauncher::FModel> Model;
	TSharedPtr<class SComboButton> ComboButton;

	UE_API TSharedRef<SWidget> MakeBuildTargetSelectionWidget();

	UE_API FText GetBuildTargetName() const;
	UE_API void SetBuildTargetName(FString BuildTargetName);


};

#undef UE_API
