// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SlateOptMacros.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "SCustomLaunchCombo"


// helper class to create a combo box that can select an item or collection of items
template<typename T>
class SCustomLaunchCombo
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, T );
	DECLARE_DELEGATE_OneParam(FOnSelectedItemsChanged, TArray<T> );
	DECLARE_DELEGATE_RetVal_OneParam(FText, FGetItemText, T)
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsItemEnabled, T)
	DECLARE_DELEGATE_RetVal_OneParam(FSlateIcon, FGetItemIcon, T)

	SLATE_BEGIN_ARGS(SCustomLaunchCombo<T>)
		: _ActionType(EUserInterfaceActionType::RadioButton)
		{}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
		SLATE_EVENT(FGetItemText, GetDisplayName)
		SLATE_EVENT(FGetItemText, GetItemToolTip)
		SLATE_EVENT(FIsItemEnabled, IsItemEnabled)
		SLATE_EVENT(FGetItemIcon, GetItemIcon)
		SLATE_EVENT(FOnSelectedItemsChanged, OnSelectedItemsChanged)
		SLATE_EVENT(FOnIsOpenChanged, OnMenuOpenChanged)
		SLATE_ATTRIBUTE(TArray<T>, Items)
		SLATE_ATTRIBUTE(T, SelectedItem)
		SLATE_ATTRIBUTE(TArray<T>, SelectedItems)
		SLATE_ARGUMENT(EUserInterfaceActionType, ActionType)
	SLATE_END_ARGS()

public:
	void Construct(	const FArguments& InArgs)
	{
		OnSelectionChanged = InArgs._OnSelectionChanged;
		OnSelectedItemsChanged = InArgs._OnSelectedItemsChanged;
		GetDisplayName = InArgs._GetDisplayName;
		GetItemToolTip = InArgs._GetItemToolTip;
		IsItemEnabled = InArgs._IsItemEnabled;
		GetItemIcon = InArgs._GetItemIcon;
		ActionType = InArgs._ActionType;

		Items = InArgs._Items;
		SelectedItem = InArgs._SelectedItem;
		SelectedItems = InArgs._SelectedItems;

		bIsMultiselect = (ActionType == EUserInterfaceActionType::Check || ActionType == EUserInterfaceActionType::ToggleButton);
		if (bIsMultiselect)
		{
			check(!OnSelectionChanged.IsBound());
			check(!SelectedItem.IsBound());
		}
		else
		{
			check(!OnSelectedItemsChanged.IsBound());
			check(!SelectedItems.IsBound());
		}

		ChildSlot
		[
			SNew(SComboButton)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(this, &SCustomLaunchCombo<T>::GetSelectedItemDisplayName)
				.Font(FCoreStyle::Get().GetFontStyle("SmallFont"))
			]
			.OnGetMenuContent(this, &SCustomLaunchCombo<T>::MakeWidget)
			.OnMenuOpenChanged(InArgs._OnMenuOpenChanged)
		];
	}

protected:
	TAttribute<T> SelectedItem;
	TAttribute<TArray<T>> SelectedItems;
	TAttribute<TArray<T>> Items;
	FOnSelectionChanged OnSelectionChanged;
	FGetItemText GetDisplayName;
	FGetItemText GetItemToolTip;
	FIsItemEnabled IsItemEnabled;
	FGetItemIcon GetItemIcon;
	FOnSelectedItemsChanged OnSelectedItemsChanged;

	bool bIsMultiselect;
	EUserInterfaceActionType ActionType;

	FText GetSelectedItemDisplayName() const
	{
		if (bIsMultiselect)
		{
			TArray<FText> SelectedItemsText;
			for (const T& Item : SelectedItems.Get())
			{
				SelectedItemsText.Add(ToText(Item));
			}

			return FText::Join(LOCTEXT("Separator",", "), SelectedItemsText);
		}

		else
		{
			T Item = SelectedItem.Get();
			return ToText(Item);
		}
	}

	void SetSelectedItem(T Value)
	{
		if (bIsMultiselect)
		{
			TArray<T> Values = SelectedItems.Get();
			if (Values.Contains(Value))
			{
				Values.Remove(Value);
			}
			else
			{
				Values.Add(Value);
			}

			OnSelectedItemsChanged.ExecuteIfBound(Values);

		}
		else
		{
			OnSelectionChanged.ExecuteIfBound(Value);
		}
	}

	ECheckBoxState GetItemCheckState(T Value)
	{
		bool bResult;
		if (bIsMultiselect)
		{
			bResult = SelectedItems.Get().Contains(Value);
		}
		else
		{
			bResult = SelectedItem.Get() == Value;
		}

		return bResult ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; 
	}

	bool GetItemIsEnabled(T Value) const
	{
		return IsItemEnabled.IsBound() ? IsItemEnabled.Execute(Value) : true;
	}

	FText ToText(T Value) const
	{
		if (GetDisplayName.IsBound())
		{
			return GetDisplayName.Execute(Value);
		}

		return ToTextOverride(Value);
	}


	virtual FText ToTextOverride(T Value) const
	{
		return FText::FromString("ERROR: GetDisplayName or ToTextOverride not bound");
	}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
	TSharedRef<SWidget> MakeWidget()
	{
		const bool bCloseOnSelection = !bIsMultiselect;
		FMenuBuilder MenuBuilder(bCloseOnSelection, nullptr);
		{
			TArray<T> MenuItems = Items.Get();

			for (const T& Item : MenuItems)
			{
				FText ItemText = ToText(Item);
				FText ToolTipText = GetItemToolTip.IsBound() ? GetItemToolTip.Execute(Item) : FText::GetEmpty();
				FSlateIcon Icon = GetItemIcon.IsBound() ? GetItemIcon.Execute(Item) : FSlateIcon();

				MenuBuilder.AddMenuEntry(
					ItemText, 
					ToolTipText, 
					Icon, 
					FUIAction(
						FExecuteAction::CreateSP(this, &SCustomLaunchCombo<T>::SetSelectedItem, Item),
						FCanExecuteAction::CreateSP(this, &SCustomLaunchCombo<T>::GetItemIsEnabled, Item),
						FGetActionCheckState::CreateSP( this, &SCustomLaunchCombo<T>::GetItemCheckState, Item)
					),
					NAME_None,
					ActionType);
			}
		}

		return MenuBuilder.MakeWidget();
	}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


};


// helper class that can display a list of items, if those items have a LexToString function available
template<typename T>
class SCustomLaunchLexToStringCombo : public SCustomLaunchCombo<T>
{
public:
	virtual FText ToTextOverride(T Value) const override
	{
		return FText::FromString(LexToString(Value));
	}
};

// helper class that can display a list of strings
class SCustomLaunchStringCombo : public SCustomLaunchCombo<FString>
{
public:
	virtual FText ToTextOverride(FString Value) const override
	{
		return FText::FromString(Value);
	}
};


#undef LOCTEXT_NAMESPACE

