// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Fonts/SlateFontInfo.h"
#include "Model/ProjectLauncherModel.h"

#define UE_API PROJECTLAUNCHER_API

class SCustomLaunchContentSchemeCombo
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, ProjectLauncher::EContentScheme );
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FGetContentSchemeAvailability, ProjectLauncher::EContentScheme, FText& )

	SLATE_BEGIN_ARGS(SCustomLaunchContentSchemeCombo) 
		: _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		{}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_EVENT(FGetContentSchemeAvailability, IsContentSchemeAvailable)
		SLATE_ATTRIBUTE(ProjectLauncher::EContentScheme, SelectedContentScheme);
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )
	SLATE_END_ARGS()

public:
	UE_API void Construct(	const FArguments& InArgs);

protected:
	TAttribute<ProjectLauncher::EContentScheme> SelectedContentScheme;
	FOnSelectionChanged OnSelectionChanged;
	FGetContentSchemeAvailability IsContentSchemeAvailable;

	UE_API TSharedRef<SWidget> MakeContentSchemeSelectionWidget();

	UE_API FText GetContentSchemeName() const;
	UE_API void SetContentScheme(ProjectLauncher::EContentScheme ContentScheme);
};

#undef UE_API
