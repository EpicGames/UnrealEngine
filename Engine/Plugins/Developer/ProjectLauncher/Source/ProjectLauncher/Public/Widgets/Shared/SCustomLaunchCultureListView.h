// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Model/ProjectLauncherModel.h"
#include <initializer_list>

#define UE_API PROJECTLAUNCHER_API

class ITableRow;
class STableViewBase;
template<typename ItemType> class STreeView;

class SCustomLaunchCultureListView
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TArray<FString> );

	SLATE_BEGIN_ARGS(SCustomLaunchCultureListView){}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_ATTRIBUTE(TArray<FString>, SelectedCultures);
		SLATE_ATTRIBUTE(FString, ProjectPath)
	SLATE_END_ARGS()

public:
	UE_API void Construct( const FArguments& InArgs, TSharedRef<ProjectLauncher::FModel> InModel );

	UE_API void RefreshCultureList();
	UE_API TSharedRef<SWidget> MakeControlsWidget();

protected:
	TSharedPtr<ProjectLauncher::FModel> Model;
	TAttribute<TArray<FString>> SelectedCultures;
	TAttribute<FString> ProjectPath;
	FOnSelectionChanged OnSelectionChanged;

private:

	struct FCultureTreeNode
	{
		FString Name;
		TArray<TSharedPtr<FCultureTreeNode>> Children;
	};
	typedef TSharedPtr<FCultureTreeNode> FCultureTreeNodePtr;

	FCultureTreeNodePtr CultureTreeRoot;

	UE_API void OnProjectChanged();

	UE_API void OnSearchFilterTextCommitted(const FText& SearchText, ETextCommit::Type InCommitType);
	UE_API void OnSearchFilterTextChanged(const FText& SearchText);

	FString CurrentFilterText;

	enum class ELocalizationGroupFilter : int32
	{
		All,
		Selected,
		EFIGS,
		CJK,
		Nordics,

		MAX
	};
	ELocalizationGroupFilter LocalizationGroupFilter;

	struct FLocalizationGroup
	{
		FLocalizationGroup( FText InDisplayName, const TArray<FString>& InLanguages )
			: DisplayName(InDisplayName)
			, Languages(InLanguages)
		{
		}

		FText DisplayName;
		const TArray<FString>& Languages;
	};
	TMap<ELocalizationGroupFilter,FLocalizationGroup> LocalizationGroupDetails;



	static const TCHAR** LocalizationGroupPrefixes[(int32)ELocalizationGroupFilter::MAX];
	static const FText LocalizationGroupDisplayName[(int32)ELocalizationGroupFilter::MAX];


	TSharedPtr<STreeView<FCultureTreeNodePtr>> CultureListView;
	TArray<FCultureTreeNodePtr> CultureListViewItemsSource;

	UE_API void GetCultureTreeNodeChildren(FCultureTreeNodePtr Item, TArray<FCultureTreeNodePtr>& OutChildren);
	UE_API TSharedRef<ITableRow> GenerateCultureTreeNodeRow(FCultureTreeNodePtr Item, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API ECheckBoxState GetCultureTreeNodeCheckState(FCultureTreeNodePtr Item) const;
	UE_API void SetCultureTreeNodeCheckState(ECheckBoxState CheckBoxState, FCultureTreeNodePtr Item);
	UE_API FText GetCultureDisplayName(FCultureTreeNodePtr Item) const;

	mutable bool bHasPaintedThisFrame = false;
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	bool bCultureListDirty = false;
	UE_API void RefreshCultureListInternal();
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
};

#undef UE_API
