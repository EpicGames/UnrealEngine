// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Shared/SCustomLaunchDeviceWidgetBase.h"

#define UE_API PROJECTLAUNCHER_API

template<typename ItemType> class SComboBox;

class SCustomLaunchDeviceCombo
	: public SCustomLaunchDeviceWidgetBase
{
public:
	SLATE_BEGIN_ARGS(SCustomLaunchDeviceCombo)
		: _AllPlatforms(false)
		{}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_ATTRIBUTE(TArray<FString>, SelectedDevices);
		SLATE_ATTRIBUTE(TArray<FString>, Platforms);
		SLATE_ARGUMENT(bool, AllPlatforms)
	SLATE_END_ARGS()

public:
	UE_API void Construct(	const FArguments& InArgs );

protected:

	UE_API TSharedRef<SWidget> GenerateDeviceListWidget(TSharedPtr<FString> DeviceID) const;
	UE_API void OnDeviceSelectionChanged(TSharedPtr<FString> DeviceID, ESelectInfo::Type InSelectInfo);
	UE_API const FSlateBrush* GetSelectedDeviceBrush() const;
	UE_API FText GetSelectedDeviceName() const;

private:
	TSharedPtr<SComboBox<TSharedPtr<FString>> > DeviceComboBox;

};

#undef UE_API
