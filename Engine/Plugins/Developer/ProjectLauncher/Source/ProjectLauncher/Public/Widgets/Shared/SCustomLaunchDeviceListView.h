// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Shared/SCustomLaunchDeviceWidgetBase.h"

#define UE_API PROJECTLAUNCHER_API

class ITableRow;
class STableViewBase;
template <typename ItemType> class SListView;

class SCustomLaunchDeviceListView
	: public SCustomLaunchDeviceWidgetBase
{
public:
	SLATE_BEGIN_ARGS(SCustomLaunchDeviceListView)
		: _AllPlatforms(false)
		, _SingleSelect(false)
		{}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_ATTRIBUTE(TArray<FString>, SelectedDevices);
		SLATE_ATTRIBUTE(TArray<FString>, Platforms);
		SLATE_ARGUMENT(bool, AllPlatforms)
		SLATE_ARGUMENT(bool, SingleSelect)
	SLATE_END_ARGS()

public:
	UE_API void Construct(	const FArguments& InArgs );

protected:
	bool bSingleSelect;

	UE_API TSharedRef<ITableRow> GenerateDeviceRow(TSharedPtr<FString> DeviceID, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API ECheckBoxState IsDeviceChecked(TSharedPtr<FString> DeviceID) const;
	UE_API void OnDeviceCheckStateChanged(ECheckBoxState NewState, TSharedPtr<FString> DeviceID);
	UE_API virtual void OnDeviceListRefreshed() override;

private:
	TSharedPtr<SListView<TSharedPtr<FString>>> DeviceListView;

};

#undef UE_API
