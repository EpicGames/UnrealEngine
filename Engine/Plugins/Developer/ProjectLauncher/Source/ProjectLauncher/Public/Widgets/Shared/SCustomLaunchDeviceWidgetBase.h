// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API PROJECTLAUNCHER_API

class ITargetDeviceProxy;
class ITargetDeviceProxyManager;

namespace PlatformInfo
{
	struct FTargetPlatformInfo;
}

class SCustomLaunchDeviceWidgetBase
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TArray<FString> );

public:
	UE_API void Construct();
	UE_API ~SCustomLaunchDeviceWidgetBase();

	UE_API void RefreshDeviceList();
	UE_API void OnSelectedPlatformChanged();

protected:
	virtual void OnDeviceListRefreshed() {};

	TAttribute<TArray<FString>> Platforms;
	TAttribute<TArray<FString>> SelectedDevices;
	FOnSelectionChanged OnSelectionChanged;
	bool bAllPlatforms = false;

	UE_API void OnDeviceProxyAdded(const TSharedRef<ITargetDeviceProxy>& DeviceProxy);
	UE_API void OnDeviceProxyRemoved(const TSharedRef<ITargetDeviceProxy>& DeviceProxy);

	UE_API const TSharedRef<ITargetDeviceProxyManager> GetDeviceProxyManager() const;

	UE_API const PlatformInfo::FTargetPlatformInfo* GetPlatformInfoForDeviceID( const FString& DeviceID ) const;
	UE_API FText GetDisplayNameForDevice( const TSharedPtr<FString>& DeviceID ) const;

	TSet<FString> KnownDeviceIDs;
	TArray<TSharedPtr<FString>> AllDeviceIDs;

};

#undef UE_API
