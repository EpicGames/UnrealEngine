// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Model/ProjectLauncherModel.h"

enum class EMapSourceFlags : uint8
{
	None                  = 0,
	MyDeveloperContent    = 1 << 0,
	OtherDeveloperContent = 1 << 1,
	ProjectPlugins        = 1 << 2,
	EngineContent         = 1 << 3,
	EnginePlugins         = 1 << 4,
};
ENUM_CLASS_FLAGS(EMapSourceFlags);

#define UE_API PROJECTLAUNCHER_API

class ITableRow;
class STableViewBase;
template<typename ItemType> class STreeView;

class SCustomLaunchMapListView
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TArray<FString> );
	DECLARE_DELEGATE_OneParam(FOnMapFilterChanged, EMapSourceFlags);

	SLATE_BEGIN_ARGS(SCustomLaunchMapListView){}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_ATTRIBUTE(TArray<FString>, SelectedMaps);
		SLATE_ATTRIBUTE(FString, ProjectPath)
		SLATE_EVENT(FOnMapFilterChanged, OnMapFilterChanged)
		SLATE_ARGUMENT(EMapSourceFlags, MapSourceFlags)
	SLATE_END_ARGS()

public:
	UE_API void Construct( const FArguments& InArgs, TSharedRef<ProjectLauncher::FModel> InModel );

	UE_API void RefreshMapList();
	UE_API TSharedRef<SWidget> MakeControlsWidget();

protected:
	TSharedPtr<ProjectLauncher::FModel> Model;
	TAttribute<TArray<FString>> SelectedMaps;
	TAttribute<FString> ProjectPath;
	FOnSelectionChanged OnSelectionChanged;
	FOnMapFilterChanged OnMapFilterChanged;

private:
	bool bShowFolders = true; // whether to display the available maps in a hierarchy or flat list

	// Content visibility flags (toggled via filter menu)
	EMapSourceFlags MapSourceFlags = EMapSourceFlags::None;

	UE_API TSharedRef<SWidget> MakeFilterMenuContent();
	bool HasAnyFilterEnabled() const;

	struct FMapTreeNode
	{
		FString Name;
		ECheckBoxState CheckBoxState = ECheckBoxState::Unchecked;
		bool bFiltered = false;
		TArray<TSharedPtr<FMapTreeNode>> Children;
	};
	typedef TSharedPtr<FMapTreeNode> FMapTreeNodePtr;

	FMapTreeNodePtr MapTreeRoot;

	UE_API void OnProjectChanged();


	UE_API void RefreshCheckBoxState( bool bExpand = false);
	UE_API ECheckBoxState RefreshCheckBoxStateRecursive(FMapTreeNodePtr Node, bool bExpand);
	UE_API void SetCheckBoxStateRecursive(FMapTreeNodePtr Node, ECheckBoxState CheckBoxState, TArray<FString>& CheckedMaps);

	UE_API void OnSearchFilterTextCommitted(const FText& SearchText, ETextCommit::Type InCommitType);
	UE_API void OnSearchFilterTextChanged(const FText& SearchText);

	FString CurrentFilterText;

	TSharedPtr<STreeView<FMapTreeNodePtr>> MapTreeView;
	TArray<FMapTreeNodePtr> MapTreeViewItemsSource;

	UE_API void GetMapTreeNodeChildren(FMapTreeNodePtr Item, TArray<FMapTreeNodePtr>& OutChildren);
	UE_API TSharedRef<ITableRow> GenerateMapTreeNodeRow(FMapTreeNodePtr Item, const TSharedRef<STableViewBase>& OwnerTable);
	UE_API ECheckBoxState GetMapTreeNodeCheckState(FMapTreeNodePtr Item) const;
	UE_API void SetMapTreeNodeCheckState(ECheckBoxState CheckBoxState, FMapTreeNodePtr Item);
	UE_API const FSlateBrush* GetMapTreeNodeIcon(FMapTreeNodePtr Node) const;
	UE_API FSlateColor GetMapTreeNodeColor(FMapTreeNodePtr Node) const;

	mutable bool bHasPaintedThisFrame = false;
	UE_API virtual int32 OnPaint( const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

	bool bMapListDirty = false;
	UE_API void RefreshMapListInternal();
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
};

#undef UE_API
