// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/PlatformMisc.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/SlateDelegates.h"

#define UE_API PROJECTLAUNCHER_API

template<typename ItemType> class SComboBox;

namespace PlatformInfo
{
	struct FTargetPlatformInfo;
}


class SCustomLaunchPlatformCombo
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, TArray<FString> );

	SLATE_BEGIN_ARGS(SCustomLaunchPlatformCombo)
		: _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		{}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_ATTRIBUTE(TArray<FString>, SelectedPlatforms);
		SLATE_ATTRIBUTE(bool, GetAdvancedPlatformsOption)
		SLATE_ATTRIBUTE(TSet<EBuildTargetType>, SupportedBuildTargets)
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )
	SLATE_END_ARGS()

public:
	UE_API void Construct(	const FArguments& InArgs);
	UE_API void RefreshPlatformsList();

protected:
	TAttribute<TArray<FString>> SelectedPlatforms;
	FOnSelectionChanged OnSelectionChanged;
	TAttribute<bool> GetAdvancedPlatformsOption;
	TAttribute<TSet<EBuildTargetType>> SupportedBuildTargets;
	TSet<FString> BuildTargetGroupStarts;

	UE_API TSharedRef<SWidget> OnGeneratePlatformListWidget( TSharedPtr<FString> Platform );
	UE_API void OnPlatformSelectionChanged( TSharedPtr<FString> Platform, ESelectInfo::Type InSelectInfo );
	UE_API const FSlateBrush* GetSelectedPlatformBrush() const;
	UE_API FText GetSelectedPlatformName() const;
	UE_API ECheckBoxState GetPlatformCheckState( const TSharedPtr<FString> Platform ) const;
	UE_API void SetPlatformCheckState( ECheckBoxState CheckState, TSharedPtr<FString> Platform );
	UE_API FSlateColor GetPlatformTextColor( const PlatformInfo::FTargetPlatformInfo* PlatformInfo );


	TArray<TSharedPtr<FString>> PlatformsList;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> PlatformsComboBox;
};

#undef UE_API
