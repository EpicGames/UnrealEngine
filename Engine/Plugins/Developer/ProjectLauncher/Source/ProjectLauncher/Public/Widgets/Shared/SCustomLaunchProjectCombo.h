// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Fonts/SlateFontInfo.h"

#define UE_API PROJECTLAUNCHER_API

class SCustomLaunchProjectCombo
	: public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, FString );

	enum class ECurrentProjectOption
	{
		None,
		ActualProject,
		Empty,
	};


	SLATE_BEGIN_ARGS(SCustomLaunchProjectCombo)
		: _ShowAnyProjectOption(false)
		, _CurrentProjectOption(ECurrentProjectOption::None)
		, _TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "NormalText" ) )
		{}
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged);
		SLATE_ATTRIBUTE(FString, SelectedProject);
		SLATE_ATTRIBUTE(bool, HasProject);
		SLATE_ARGUMENT(bool, ShowAnyProjectOption)
		SLATE_ARGUMENT(ECurrentProjectOption, CurrentProjectOption)
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)
		SLATE_STYLE_ARGUMENT( FTextBlockStyle, TextStyle )
	SLATE_END_ARGS()

public:
	UE_API void Construct(	const FArguments& InArgs);

protected:
	TAttribute<FString> SelectedProject;
	TAttribute<bool> HasProject;
	FOnSelectionChanged OnSelectionChanged;
	bool bShowAnyProjectOption;
	ECurrentProjectOption CurrentProjectOption;

	UE_API TSharedRef<SWidget> MakeProjectSelectionWidget();
	UE_API void OnBrowseForProject();

	UE_API FText GetProjectName() const;
	UE_API void SetProjectPath(FString ProjectPath);


};

#undef UE_API
